[CoreRedirects]
+PropertyRedirects=(OldName="TimecodeSynchronizer.MasterSynchronizationSourceIndex",NewName="TimecodeSynchronizer.MainSynchronizationSourceIndex")