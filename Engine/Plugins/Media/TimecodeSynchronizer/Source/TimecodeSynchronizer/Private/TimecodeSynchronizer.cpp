// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimecodeSynchronizer.h"
#include "TimecodeSynchronizerModule.h"

#include "Engine/Engine.h"
#include "FixedFrameRateCustomTimeStep.h"
#include "Misc/App.h"

#if WITH_EDITOR
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif //WITH_EDITOR

#define LOCTEXT_NAMESPACE "TimecodeSynchronizer"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

namespace TimecodeSynchronizerPrivate
{
	struct FTimecodeInputSourceValidator
	{
	private:

		const FTimecodeSynchronizerCachedSyncState& SyncState;

		bool bTimecodeErrors = false;
		int32 FoundOffset = 0;

		FFrameTime Newest;
		FFrameTime Oldest;

		bool bAnySourcesHadRollover = false;
		bool bAllSourcesHadRollover = false;

	public:

		FTimecodeInputSourceValidator(const FTimecodeSynchronizerCachedSyncState& InSyncState, const FTimecodeSynchronizerActiveTimecodedInputSource& InitialInputSource) :
			SyncState(InSyncState)
		{
			ValidateSource(InitialInputSource);
			if (AllSourcesAreValid())
			{
				const FTimecodeSourceState& SynchronizerRelativeState = InitialInputSource.GetSynchronizerRelativeState();
				Newest = SynchronizerRelativeState.NewestAvailableSample;
				Oldest = SynchronizerRelativeState.OldestAvailableSample;
				bAnySourcesHadRollover = (SyncState.RolloverFrame.IsSet() && Newest < Oldest);
				bAllSourcesHadRollover = bAnySourcesHadRollover;
			}
		}

		void UpdateFrameTimes(const FTimecodeSynchronizerActiveTimecodedInputSource& InputSource)
		{
			ValidateSource(InputSource);
			if (AllSourcesAreValid())
			{
				const FTimecodeSourceState& SynchronizerRelativeState = InputSource.GetSynchronizerRelativeState();
				Oldest = FMath::Max(SynchronizerRelativeState.OldestAvailableSample, Oldest);
				Newest = FMath::Min(SynchronizerRelativeState.NewestAvailableSample, Newest);
			}
		}

		const bool AllSourcesAreValid() const
		{
			return !FoundTimecodeErrors() && !FoundFrameRolloverMistmatch();
		}

		const bool FoundFrameRolloverMistmatch() const
		{
			return bAllSourcesHadRollover != bAnySourcesHadRollover;
		}

		const bool FoundTimecodeErrors() const
		{
			return bTimecodeErrors;
		}

		const bool DoAllSourcesContainFrame(const FFrameTime& FrameToCheck) const
		{
			if (FoundTimecodeErrors() || FoundFrameRolloverMistmatch())
			{
				return false;
			}
			else if (!SyncState.RolloverFrame.IsSet() || !bAnySourcesHadRollover)
			{
				return (Oldest <= FrameToCheck) && (FrameToCheck <= Newest);
			}
			else
			{
				return UTimeSynchronizationSource::IsFrameBetweenWithRolloverModulus(FrameToCheck, Oldest, Newest, SyncState.RolloverFrame.GetValue());
			}
		}

		const int32 CalculateOffsetNewest(const FFrameTime& FrameTime) const
		{
			// These cases should never happen, but they may be recoverable, so don't crash.
			ensureAlwaysMsgf(!FoundTimecodeErrors(), TEXT("FTimecodeInputSourceValidator::CalculateOffsetNewest - Called with TimecodeErrors"));
			ensureAlwaysMsgf(!FoundFrameRolloverMistmatch(), TEXT("FTimecodeInputSourceValidater::CalculateOffsetNewest - Called with FrameRolloverMismatch"));

			bool bUnused_DidRollover;
			return UTimeSynchronizationSource::FindDistanceBetweenFramesWithRolloverModulus(FrameTime, Newest, SyncState.RolloverFrame, bUnused_DidRollover);
		}

		const int32 CalculateOffsetOldest(const FFrameTime& FrameTime) const
		{
			// These cases should never happen, but they may be recoverable, so don't crash.
			ensureAlwaysMsgf(!FoundTimecodeErrors(), TEXT("FTimecodeInputSourceValidator::CalculateOffsetOldest - Called with TimecodeErrors"));
			ensureAlwaysMsgf(!FoundFrameRolloverMistmatch(), TEXT("FTimecodeInputSourceValidater::CalculateOffsetOldest - Called with FrameRolloverMismatch"));

			bool bUnused_DidRollover;

			// Because we switched order of inputs, we need to flip the output as well.
			return -UTimeSynchronizationSource::FindDistanceBetweenFramesWithRolloverModulus(Oldest, FrameTime, SyncState.RolloverFrame, bUnused_DidRollover);
		}

	private:

		void ValidateSource(const FTimecodeSynchronizerActiveTimecodedInputSource& InputSource)
		{
			const FTimecodeSourceState& SynchronizerRelativeState = InputSource.GetSynchronizerRelativeState();
			const FFrameTime& OldestSample = SynchronizerRelativeState.OldestAvailableSample;
			const FFrameTime& NewestSample = SynchronizerRelativeState.NewestAvailableSample;

			const bool bUseRollover = SyncState.RolloverFrame.IsSet();
			const bool bSourceBufferHasRolledOver = (bUseRollover && OldestSample > NewestSample);

			if (!bUseRollover)
			{
				// If we're not using rollover, but Oldest time is later than the Newest time, then the source is
				// reporting incorrect values.
				if (OldestSample > NewestSample)
				{
					UE_LOGF(LogTimecodeSynchronizer, Warning, "Source %ls reported out of order frame times (Oldest = %d | Newest = %d)",
						*InputSource.GetDisplayName(), OldestSample.GetFrame().Value, NewestSample.GetFrame().Value);

					bTimecodeErrors = true;
				}
			}
			else
			{
				const FFrameTime& RolloverFrame = SyncState.RolloverFrame.GetValue();

				// If we're using rollover, and either source has reported a value beyond where we expect to rollover,
				// then the source is reporting incorrect values.
				if ((OldestSample >= RolloverFrame) || (NewestSample >= RolloverFrame))
				{
					UE_LOGF(LogTimecodeSynchronizer, Warning, "Source %ls reported frames that go beyond expected rollover point (Oldest = %d | Newest = %d | Rollover = %d",
						*InputSource.GetDisplayName(), OldestSample.GetFrame().Value, NewestSample.GetFrame().Value, RolloverFrame.GetFrame().Value);

					bTimecodeErrors = true;
				}

				if (bSourceBufferHasRolledOver)
				{
					// See CalculateOffset for the justification

					// Since we think a rollover has occurred, then we'd expect the frame values to be relatively
					// far apart.
					const int32 Offset = (OldestSample - NewestSample).GetFrame().Value;
					if (FMath::Abs<int32>(Offset) < (RolloverFrame.GetFrame().Value / 2))
					{
						UE_LOGF(LogTimecodeSynchronizer, Warning, "Source %ls reported out of order frame times (Oldest = %d | Newest = %d)",
							*InputSource.GetDisplayName(), OldestSample.GetFrame().Value, NewestSample.GetFrame().Value);

						bTimecodeErrors = true;
					}
				}
			}

			bAllSourcesHadRollover &= bSourceBufferHasRolledOver;
			bAnySourcesHadRollover |= bSourceBufferHasRolledOver;
		}
	};

	void ShowSlateNotification()
	{
#if WITH_EDITOR
		if (GIsEditor)
		{
			FNotificationInfo NotificationInfo(LOCTEXT("TimecodeSynchronizerError", "The synchronization failed. Check the Output Log for details."));
			NotificationInfo.ExpireDuration = 2.0f;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
#endif // WITH_EDITOR
	}
}

/**
 * UTimecodeSynchronizer
 */

UTimecodeSynchronizer::UTimecodeSynchronizer()
	: FixedFrameRate(30, 1)
	, TimecodeProviderType(ETimecodeSynchronizationTimecodeType::TimecodeProvider)
	, TimecodeProvider(nullptr)
	, MainSynchronizationSourceIndex(INDEX_NONE)
	, PreRollingTimecodeMarginOfErrors(4)
	, PreRollingTimeout(30.f)
	, bIsTickEnabled(false)
	, State(ESynchronizationState::None)
	, StartPreRollingTime(0.0)
	, bRegistered(false)
	, PreviousFixedFrameRate(0.f)
	, bPreviousUseFixedFrameRate(false)
	, ActiveMainSynchronizationTimecodedSourceIndex(INDEX_NONE)
	, bFailGuard(false)
	, bAddSourcesGuard(false)
	, bShouldResetTimecodeProvider(false)
{
}

void UTimecodeSynchronizer::BeginDestroy()
{
	Super::BeginDestroy();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		Unregister();
	}
}

#if WITH_EDITOR
bool UTimecodeSynchronizer::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	const FName PropertyName = InProperty->GetFName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTimecodeSynchronizer, TimecodeProvider))
	{
		return TimecodeProviderType == ETimecodeSynchronizationTimecodeType::TimecodeProvider;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UTimecodeSynchronizer, FixedFrameRate))
	{
		return FrameRateSource == ETimecodeSynchronizationFrameRateSources::CustomFrameRate;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UTimecodeSynchronizer, MainSynchronizationSourceIndex))
	{
		return TimecodeProviderType == ETimecodeSynchronizationTimecodeType::InputSource;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UTimecodeSynchronizer, FrameOffset))
	{
		return SyncMode == ETimecodeSynchronizationSyncMode::UserDefinedOffset;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UTimecodeSynchronizer, AutoFrameOffset))
	{
		return (SyncMode == ETimecodeSynchronizationSyncMode::Auto) ||
			(SyncMode == ETimecodeSynchronizationSyncMode::AutoOldest);
	}

	return true;
}

void UTimecodeSynchronizer::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	// Make sure the main source index is valid
	if (TimecodeProviderType == ETimecodeSynchronizationTimecodeType::InputSource)
	{
		if (!TimeSynchronizationInputSources.IsValidIndex(MainSynchronizationSourceIndex)
			|| TimeSynchronizationInputSources[MainSynchronizationSourceIndex] == nullptr
			|| !TimeSynchronizationInputSources[MainSynchronizationSourceIndex]->bUseForSynchronization)
		{
			MainSynchronizationSourceIndex = INDEX_NONE;
			UE_LOGF(LogTimecodeSynchronizer, Warning, "The MainSynchronizationSourceIndex is not valid.");
		}
	}

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}
#endif

FQualifiedFrameTime UTimecodeSynchronizer::GetQualifiedFrameTime() const
{
	return FQualifiedFrameTime(GetTimecodeInternal(), GetFrameRateInternal());
}

FTimecode UTimecodeSynchronizer::GetTimecodeInternal() const
{
	FTimecode Timecode;
	if (IsSynchronized())
	{
		Timecode = UTimeSynchronizationSource::ConvertFrameTimeToTimecode(CurrentSystemFrameTime.GetValue(), CachedSyncState.FrameRate);
	}
	else if (IsSynchronizing())
	{
		Timecode = UTimeSynchronizationSource::ConvertFrameTimeToTimecode(CurrentProviderFrameTime, CachedSyncState.FrameRate);
	}
	else
	{
		Timecode = UTimeSynchronizationSource::ConvertFrameTimeToTimecode(GetProviderFrameTime(), GetFrameRateInternal());
	}

	return Timecode;
}

FFrameTime UTimecodeSynchronizer::GetProviderFrameTime() const
{
	FFrameTime ProviderFrameTime;

	if (TimecodeProviderType == ETimecodeSynchronizationTimecodeType::InputSource)
	{
		if (SynchronizedSources.IsValidIndex(ActiveMainSynchronizationTimecodedSourceIndex))
		{
			const FTimecodeSynchronizerActiveTimecodedInputSource& TimecodedInputSource = SynchronizedSources[ActiveMainSynchronizationTimecodedSourceIndex];

			if (GFrameCounter != LastUpdatedSources)
			{
				const_cast<FTimecodeSynchronizerActiveTimecodedInputSource&>(TimecodedInputSource).UpdateSourceState(GetFrameRateInternal());
			}

			if (TimecodedInputSource.IsReady())
			{
				ProviderFrameTime = TimecodedInputSource.GetSynchronizerRelativeState().NewestAvailableSample;
			}
			else
			{
				UE_LOGF(LogTimecodeSynchronizer, Log, "Unable to get frame time - Specified source was not ready.");
			}
		}
		else
		{
			UE_LOGF(LogTimecodeSynchronizer, Log, "Unable to get frame time - Invalid source specified.");
		}
	}
	else if (CachedProxiedTimecodeProvider)
	{
		ProviderFrameTime = FFrameTime(CachedProxiedTimecodeProvider->GetTimecode().ToFrameNumber(GetFrameRateInternal()));
	}

	return ProviderFrameTime;
}

FFrameRate UTimecodeSynchronizer::GetFrameRateInternal() const
{
	return RegisteredCustomTimeStep ? RegisteredCustomTimeStep->GetFixedFrameRate() : FixedFrameRate;
}

ETimecodeProviderSynchronizationState UTimecodeSynchronizer::GetSynchronizationState() const
{
	switch (State)
	{
	case ESynchronizationState::Initializing:
	case ESynchronizationState::PreRolling_WaitGenlockTimecodeProvider:
	case ESynchronizationState::PreRolling_WaitReadiness:
	case ESynchronizationState::PreRolling_Synchronizing:
		return ETimecodeProviderSynchronizationState::Synchronizing;
	case ESynchronizationState::Synchronized:
		return ETimecodeProviderSynchronizationState::Synchronized;
	case ESynchronizationState::Error:
		return ETimecodeProviderSynchronizationState::Error;
	}
	return ETimecodeProviderSynchronizationState::Closed;
}

bool UTimecodeSynchronizer::Initialize(class UEngine* InEngine)
{
	// The engine only allows one provider to be active at a given time.
	// However, we are a special case just acting as a pass through.
	// Therefore, we need to make sure we pass along the initialization / shutdown requests.
	if (CachedProxiedTimecodeProvider)
	{
		return CachedProxiedTimecodeProvider->Initialize(InEngine);
	}

	return true;
}

void UTimecodeSynchronizer::Shutdown(class UEngine* InEngine)
{
	if (CachedProxiedTimecodeProvider)
	{
		CachedProxiedTimecodeProvider->Shutdown(InEngine);
	}
}

bool UTimecodeSynchronizer::IsSynchronizing() const
{
	return State == ESynchronizationState::PreRolling_WaitGenlockTimecodeProvider
		|| State == ESynchronizationState::PreRolling_WaitReadiness
		|| State == ESynchronizationState::PreRolling_Synchronizing
		|| State == ESynchronizationState::Initializing;
}

bool UTimecodeSynchronizer::IsSynchronized() const
{
	return State == ESynchronizationState::Synchronized;
}

bool UTimecodeSynchronizer::IsError() const
{
	return State == ESynchronizationState::Error;
}

void UTimecodeSynchronizer::Register()
{
	if (!bRegistered)
	{
		bRegistered = true;
		bShouldResetTimecodeProvider = false;

		if (FrameRateSource == ETimecodeSynchronizationFrameRateSources::EngineCustomTimeStepFrameRate)
		{
			if (GEngine->GetCustomTimeStep() == nullptr)
			{
				UE_LOGF(LogTimecodeSynchronizer, Error, "Engine does not have Genlock in place.");
				SwitchState(ESynchronizationState::Error);
				return;
			}

			UFixedFrameRateCustomTimeStep* FixedFrameRateCustomTimeStep = Cast< UFixedFrameRateCustomTimeStep>(GEngine->GetCustomTimeStep());

			if (FixedFrameRateCustomTimeStep == nullptr)
			{
				UE_LOGF(LogTimecodeSynchronizer, Error, "Engine CustomTimeStep must be a FixedFrameRateCustomTimeStep.");
				SwitchState(ESynchronizationState::Error);
				return;
			}

			RegisteredCustomTimeStep = FixedFrameRateCustomTimeStep;
		}
		else
		{
			PreviousFixedFrameRate = GEngine->FixedFrameRate;
			bPreviousUseFixedFrameRate = GEngine->bUseFixedFrameRate;
			GEngine->FixedFrameRate = FixedFrameRate.AsDecimal();
			GEngine->bUseFixedFrameRate = true;
		}

		CachedPreviousTimecodeProvider = GEngine->GetTimecodeProvider();
		if (TimecodeProviderType == ETimecodeSynchronizationTimecodeType::TimecodeProvider)
		{
			CachedProxiedTimecodeProvider = TimecodeProvider;
		}
		else if (TimecodeProviderType == ETimecodeSynchronizationTimecodeType::DefaultProvider)
		{
			CachedProxiedTimecodeProvider = GEngine->GetTimecodeProvider();
		}
		else
		{
			CachedProxiedTimecodeProvider = nullptr;
		}

		bShouldResetTimecodeProvider = true;
		if (!GEngine->SetTimecodeProvider(this))
		{
			UE_LOGF(LogTimecodeSynchronizer, Error, "TimecodeProvider failed to be set on Engine.");
			SwitchState(ESynchronizationState::Error);
			return;
		}

		GEngine->OnTimecodeProviderChanged().AddUObject(this, &UTimecodeSynchronizer::OnTimecodeProviderChanged);

		SetTickEnabled(true);
	}
}

void UTimecodeSynchronizer::Unregister()
{
	if (bRegistered)
	{
		bRegistered = false;

		GEngine->OnTimecodeProviderChanged().RemoveAll(this);
		if (GEngine->GetTimecodeProvider() == this)
		{
			GEngine->SetTimecodeProvider(bShouldResetTimecodeProvider ? CachedPreviousTimecodeProvider : nullptr);
			bShouldResetTimecodeProvider = false;
		}
		CachedPreviousTimecodeProvider = nullptr;
		CachedProxiedTimecodeProvider = nullptr;

		if (RegisteredCustomTimeStep == nullptr)
		{
			GEngine->FixedFrameRate = PreviousFixedFrameRate;
			GEngine->bUseFixedFrameRate = bPreviousUseFixedFrameRate;
		}
		RegisteredCustomTimeStep = nullptr;

		SetTickEnabled(false);
	}
}

void UTimecodeSynchronizer::OnTimecodeProviderChanged()
{
	if (bRegistered && !IsError())
	{
		bShouldResetTimecodeProvider = false;
		UE_LOGF(LogTimecodeSynchronizer, Error, "The Engine's TimecodeProvider changed.");
		SwitchState(ESynchronizationState::Error);
		return;
	}
}

void UTimecodeSynchronizer::SetTickEnabled(bool bEnabled)
{
	bIsTickEnabled = bEnabled;
}

bool UTimecodeSynchronizer::IsTickable() const
{
	return bIsTickEnabled;
}

void UTimecodeSynchronizer::Tick(float DeltaTime)
{
	UpdateSourceStates();
	CurrentProviderFrameTime = GetProviderFrameTime();

	Tick_Switch();

	if (IsSynchronizing() && bUsePreRollingTimeout)
	{
		const double TimeSinceStarted = FApp::GetCurrentTime() - StartPreRollingTime;
		if (TimeSinceStarted > PreRollingTimeout)
		{
			UE_LOGF(LogTimecodeSynchronizer, Error, "PreRoll Timeout.");
			SwitchState(ESynchronizationState::Error);
		}
	}
}

bool UTimecodeSynchronizer::StartSynchronization()
{
	if (IsSynchronizing() || IsSynchronized())
	{
		UE_LOGF(LogTimecodeSynchronizer, Log, "Already synchronizing or synchronized.");
		return true;
	}
	else
	{
		if (!ensure(SynchronizedSources.Num() == 0) || !ensure(NonSynchronizedSources.Num() == 0) || !ensure(ActiveMainSynchronizationTimecodedSourceIndex))
		{
			UE_LOGF(LogTimecodeSynchronizer, Error, "StartSynchronization called without properly closing sources");
			CloseSources();
		}

		DynamicSources.Reset();
		SwitchState(ESynchronizationState::Initializing);
		OpenSources();

		if (SynchronizedSources.Num() == 0)
		{
			UE_LOGF(LogTimecodeSynchronizer, Warning, "No sources available to synchronize.");
			SwitchState(ESynchronizationState::Error);
		}
		else if (TimecodeProviderType == ETimecodeSynchronizationTimecodeType::InputSource && ActiveMainSynchronizationTimecodedSourceIndex == INDEX_NONE)
		{
			UE_LOGF(LogTimecodeSynchronizer, Warning, "The Main Synchronization Source could not be found.");
			SwitchState(ESynchronizationState::Error);
		}
		else
		{
			Register();

			if (bRegistered)
			{
				SwitchState(ESynchronizationState::PreRolling_WaitGenlockTimecodeProvider);
			}
		}

		return bRegistered;
	}
}

void UTimecodeSynchronizer::StopSynchronization()
{
	if (IsSynchronizing() || IsSynchronized() || IsError())
	{
		Unregister();
		CloseSources();

		LastUpdatedSources = 0;
		CurrentSystemFrameTime.Reset();
		CurrentProviderFrameTime = FFrameTime(0);
		StartPreRollingTime = 0.f;
		DynamicSources.Reset();

		SynchronizationEvent.Broadcast(bFailGuard ? ETimecodeSynchronizationEvent::SynchronizationFailed : ETimecodeSynchronizationEvent::SynchronizationStopped);
		SwitchState(ESynchronizationState::None);
	}
}

void UTimecodeSynchronizer::AddRuntimeTimeSynchronizationSource(UTimeSynchronizationSource* Source)
{
	if (Source && bAddSourcesGuard)
	{
		DynamicSources.AddUnique(Source);
	}
}

void UTimecodeSynchronizer::SwitchState(const ESynchronizationState NewState)
{
	if (NewState != State)
	{
		State = NewState;

		// Do any setup that needs to happen to "enter" the state.
		switch (NewState)
		{
		case ESynchronizationState::Initializing:
			CachedSyncState.FrameRate = GetFrameRate();
			CachedSyncState.SyncMode = SyncMode;
			CachedSyncState.FrameOffset = FrameOffset;

			// System time inherently has rollover.
			if (bWithRollover)
			{
				// In most cases, rollover occurs on 24 periods.
				// TODO: Make this configurable
				CachedSyncState.RolloverFrame = FTimecode(24, 0, 0, 0, false).ToFrameNumber(CachedSyncState.FrameRate);
			}
			else
			{
				CachedSyncState.RolloverFrame.Reset();
			}

			{
				TGuardValue<bool> LocalAddSourcesGuard(bAddSourcesGuard, true);
				SynchronizationEvent.Broadcast(ETimecodeSynchronizationEvent::SynchronizationStarted);
			}
			
			break;

		case ESynchronizationState::PreRolling_WaitGenlockTimecodeProvider:
			StartPreRollingTime = FApp::GetCurrentTime();
			break;

		case ESynchronizationState::Synchronized:
			StartSources();
			SynchronizationEvent.Broadcast(ETimecodeSynchronizationEvent::SynchronizationSucceeded);
			break;

		case ESynchronizationState::Error:
		{
			TGuardValue<bool> FailScope(bFailGuard, true);
			StopSynchronization();
			TimecodeSynchronizerPrivate::ShowSlateNotification();
			break;
		}

		default:
			break;
		};

		Tick_Switch();
	}
}

void UTimecodeSynchronizer::Tick_Switch()
{
#define CONDITIONALLY_CALL_TICK(TickFunc) {if (ShouldTick()) {TickFunc();}}

	switch (State)
	{
	case ESynchronizationState::Initializing:
		break;

	case ESynchronizationState::PreRolling_WaitGenlockTimecodeProvider:
		CONDITIONALLY_CALL_TICK(TickPreRolling_WaitGenlockTimecodeProvider);
		break;

	case ESynchronizationState::PreRolling_WaitReadiness:
		CONDITIONALLY_CALL_TICK(TickPreRolling_WaitReadiness);
		break;

	case ESynchronizationState::PreRolling_Synchronizing:
		CONDITIONALLY_CALL_TICK(TickPreRolling_Synchronizing);
		break;

	case ESynchronizationState::Synchronized:
		CONDITIONALLY_CALL_TICK(Tick_Synchronized);
		break;

	default:
		SetTickEnabled(false);
		break;
	}

#undef CONDITIONALLY_CALL_TICK
}

bool UTimecodeSynchronizer::ShouldTick()
{
	return Tick_TestGenlock() && Tick_TestTimecode();
}

bool UTimecodeSynchronizer::Tick_TestGenlock()
{
	if (FrameRateSource == ETimecodeSynchronizationFrameRateSources::EngineCustomTimeStepFrameRate)
	{
		if (RegisteredCustomTimeStep == nullptr)
		{
			UE_LOGF(LogTimecodeSynchronizer, Error, "The registered Genlock source is invalid.");
			SwitchState(ESynchronizationState::Error);
			return false;
		}

		if (GEngine->GetCustomTimeStep() != RegisteredCustomTimeStep)
		{
			UE_LOGF(LogTimecodeSynchronizer, Error, "The registered Genlock source is not the Engine CustomTimeStep.");
			SwitchState(ESynchronizationState::Error);
			return false;
		}

		const ECustomTimeStepSynchronizationState SynchronizationState = RegisteredCustomTimeStep->GetSynchronizationState();
		if (SynchronizationState != ECustomTimeStepSynchronizationState::Synchronized && SynchronizationState != ECustomTimeStepSynchronizationState::Synchronizing)
		{
			UE_LOGF(LogTimecodeSynchronizer, Error, "The Genlock source stopped while synchronizing.");
			SwitchState(ESynchronizationState::Error);
			return false;
		}

		return SynchronizationState == ECustomTimeStepSynchronizationState::Synchronized;
	}
	return true;
}

bool UTimecodeSynchronizer::Tick_TestTimecode()
{
	if (GEngine->GetTimecodeProvider() != this)
	{
		UE_LOGF(LogTimecodeSynchronizer, Error, "The registered TimecodeProvider is not the Engine TimecodeProvider.");
		SwitchState(ESynchronizationState::Error);
		return false;
	}

	if (TimecodeProviderType == ETimecodeSynchronizationTimecodeType::InputSource)
	{
		if (!SynchronizedSources.IsValidIndex(ActiveMainSynchronizationTimecodedSourceIndex))
		{
			UE_LOGF(LogTimecodeSynchronizer, Error, "The InputSource '%d' that we try to synchronize on is not valid.", ActiveMainSynchronizationTimecodedSourceIndex);
			SwitchState(ESynchronizationState::Error);
			return false;
		}

		return SynchronizedSources[ActiveMainSynchronizationTimecodedSourceIndex].IsReady();
	}
	else 
	{
		if (CachedProxiedTimecodeProvider == nullptr)
		{
			UE_LOGF(LogTimecodeSynchronizer, Error, "The TimecodeProvider became invalid while synchronizing.");
			SwitchState(ESynchronizationState::Error);
			return false;
		}

		const ETimecodeProviderSynchronizationState SynchronizationState = CachedProxiedTimecodeProvider->GetSynchronizationState();
		if (SynchronizationState != ETimecodeProviderSynchronizationState::Synchronized && SynchronizationState != ETimecodeProviderSynchronizationState::Synchronizing)
		{
			UE_LOGF(LogTimecodeSynchronizer, Error, "The TimecodeProvider stopped while synchronizing.");
			SwitchState(ESynchronizationState::Error);
			return false;
		}

		if (CachedProxiedTimecodeProvider->GetFrameRate() != GetFrameRate())
		{
			UE_LOGF(LogTimecodeSynchronizer, Error, "The TimecodeProvider frame rate do not correspond to the specified frame rate.");
			SwitchState(ESynchronizationState::Error);
		}

		return SynchronizationState == ETimecodeProviderSynchronizationState::Synchronized;
	}
}

void UTimecodeSynchronizer::TickPreRolling_WaitGenlockTimecodeProvider()
{
	SwitchState(ESynchronizationState::PreRolling_WaitReadiness);
}

void UTimecodeSynchronizer::TickPreRolling_WaitReadiness()
{
	bool bAllSourceAreReady = true;

	for (const FTimecodeSynchronizerActiveTimecodedInputSource& InputSource : SynchronizedSources)
	{
		if (InputSource.IsReady())
		{
			const FFrameRate SourceFrameRate = InputSource.GetFrameRate();
			if (!SourceFrameRate.IsMultipleOf(CachedSyncState.FrameRate) && !SourceFrameRate.IsFactorOf(CachedSyncState.FrameRate))
			{
				UE_LOGF(LogTimecodeSynchronizer, Warning, "Source %ls doesn't have a frame rate common to TimecodeSynchronizer frame rate.", *InputSource.GetDisplayName())
			}
		}
		else
		{
			bAllSourceAreReady = false;
		}
	}

	if (bAllSourceAreReady)
	{
		SwitchState(ESynchronizationState::PreRolling_Synchronizing);
	}
}

void UTimecodeSynchronizer::TickPreRolling_Synchronizing()
{
	TimecodeSynchronizerPrivate::FTimecodeInputSourceValidator Validator(CachedSyncState, SynchronizedSources[0]);
	for (int32 i = 1; i < SynchronizedSources.Num(); ++i)
	{
		Validator.UpdateFrameTimes(SynchronizedSources[i]);
	}

	if (Validator.AllSourcesAreValid())
	{
		switch (CachedSyncState.SyncMode)
		{
		case ETimecodeSynchronizationSyncMode::Auto:
			ActualFrameOffset = Validator.CalculateOffsetNewest(CurrentProviderFrameTime) - AutoFrameOffset;
			break;

		case ETimecodeSynchronizationSyncMode::AutoOldest:
			ActualFrameOffset = Validator.CalculateOffsetOldest(CurrentProviderFrameTime) + AutoFrameOffset;
			break;

		default:
			ActualFrameOffset = CachedSyncState.FrameOffset;
			break;
		}

		if (Validator.DoAllSourcesContainFrame(CalculateSyncTime()))
		{
			SwitchState(ESynchronizationState::Synchronized);
		}
	}
}

void UTimecodeSynchronizer::Tick_Synchronized()
{
	// Sanity check to make sure all sources still have valid frames.
	CurrentSystemFrameTime = CalculateSyncTime();
	const FFrameTime& UseFrameTime = CurrentSystemFrameTime.GetValue();

	if (CachedSyncState.RolloverFrame.IsSet())
	{
		for (const FTimecodeSynchronizerActiveTimecodedInputSource& InputSource : SynchronizedSources)
		{
			const FTimecodeSourceState& SynchronizerRelativeState = InputSource.GetSynchronizerRelativeState();
			if (!UTimeSynchronizationSource::IsFrameBetweenWithRolloverModulus(UseFrameTime, SynchronizerRelativeState.OldestAvailableSample, SynchronizerRelativeState.NewestAvailableSample, CachedSyncState.RolloverFrame.GetValue()))
			{
				UE_LOGF(LogTimecodeSynchronizer, Warning, "Source '%ls' doesn't have the timecode ready.", *InputSource.GetDisplayName());
			}
		}
	}
	else
	{
		for (const FTimecodeSynchronizerActiveTimecodedInputSource& InputSource : SynchronizedSources)
		{
			const FTimecodeSourceState& SynchronizerRelativeState = InputSource.GetSynchronizerRelativeState();
			if (SynchronizerRelativeState.OldestAvailableSample > UseFrameTime || UseFrameTime > SynchronizerRelativeState.NewestAvailableSample)
			{
				UE_LOGF(LogTimecodeSynchronizer, Warning, "Source '%ls' doesn't have the timecode ready.", *InputSource.GetDisplayName());
			}
		}
	}
}

const bool FTimecodeSynchronizerActiveTimecodedInputSource::UpdateSourceState(const FFrameRate& SynchronizerFrameRate)
{
	check(InputSource);

	bIsReady = InputSource->IsReady() && InputSource->GetFrameRate().IsValid() && InputSource->GetFrameRate().Numerator > 0;

	if (bIsReady)
	{
		FrameRate = InputSource->GetFrameRate();

		InputSourceState.NewestAvailableSample = InputSource->GetNewestSampleTime();
		InputSourceState.OldestAvailableSample = InputSource->GetOldestSampleTime();

		if (FrameRate != SynchronizerFrameRate)
		{
			SynchronizerRelativeState.NewestAvailableSample = FFrameRate::TransformTime(InputSourceState.NewestAvailableSample, FrameRate, SynchronizerFrameRate);
			SynchronizerRelativeState.OldestAvailableSample = FFrameRate::TransformTime(InputSourceState.OldestAvailableSample, FrameRate, SynchronizerFrameRate);
		}
		else
		{
			SynchronizerRelativeState = InputSourceState;
		}
	}

	return bIsReady;
}

void UTimecodeSynchronizer::StartSources()
{
	FTimeSynchronizationStartData StartData;
	CurrentSystemFrameTime = StartData.StartFrame = CalculateSyncTime();

	FApp::SetCurrentFrameTime(GetDelayedQualifiedFrameTime());

	for (UTimeSynchronizationSource* InputSource : TimeSynchronizationInputSources)
	{
		if (InputSource != nullptr)
		{
			InputSource->Start(StartData);
		}
	}

	for (UTimeSynchronizationSource* InputSource : DynamicSources)
	{
		if (InputSource != nullptr)
		{
			InputSource->Start(StartData);
		}
	}
}

void UTimecodeSynchronizer::OpenSources()
{
	FTimeSynchronizationOpenData OpenData;
	OpenData.RolloverFrame = CachedSyncState.RolloverFrame;
	OpenData.SynchronizationFrameRate = CachedSyncState.FrameRate;
	for (int32 Index = 0; Index < TimeSynchronizationInputSources.Num(); ++Index)
	{
		if (UTimeSynchronizationSource* InputSource = TimeSynchronizationInputSources[Index])
		{
			if (InputSource->Open(OpenData))
			{
				if (InputSource->bUseForSynchronization)
				{
					FTimecodeSynchronizerActiveTimecodedInputSource& NewSource = SynchronizedSources.Emplace_GetRef(InputSource);
					if (TimecodeProviderType == ETimecodeSynchronizationTimecodeType::InputSource && Index == MainSynchronizationSourceIndex)
					{
						ActiveMainSynchronizationTimecodedSourceIndex = SynchronizedSources.Num() - 1;
					}
				}
				else
				{
					NonSynchronizedSources.Emplace(InputSource);
				}
			}
			else
			{
				UE_LOGF(LogTimecodeSynchronizer, Warning, "Source %d, could not be open.", Index);
			}
		}
	}

	// Don't consider dynamic sources for the main synchronization index.
	for (UTimeSynchronizationSource* InputSource : DynamicSources)
	{
		if (InputSource != nullptr)
		{
			if (InputSource->Open(OpenData))
			{
				(InputSource->bUseForSynchronization ? SynchronizedSources : NonSynchronizedSources).Emplace(InputSource);
			}
		}
	}
}

void UTimecodeSynchronizer::CloseSources()
{
	for (UTimeSynchronizationSource* InputSource : TimeSynchronizationInputSources)
	{
		if (InputSource != nullptr)
		{
			InputSource->Close();
		}
	}

	for (UTimeSynchronizationSource* InputSource : DynamicSources)
	{
		if (InputSource != nullptr)
		{
			InputSource->Close();
		}
	}

	SynchronizedSources.Reset();
	NonSynchronizedSources.Reset();
	ActiveMainSynchronizationTimecodedSourceIndex = INDEX_NONE;
}

void UTimecodeSynchronizer::UpdateSourceStates()
{
	// Update all of our source states.
	if (GFrameCounter != LastUpdatedSources)
	{
		LastUpdatedSources = GFrameCounter;

		// If we're in the process of synchronizing, or have already achieved synchronization,
		// we don't expect sources to become unready. If they do, that's an error.
		// This is only relevant to 
		const bool bTreatUnreadyAsError = (State > ESynchronizationState::PreRolling_WaitReadiness);
		TArray<const FTimecodeSynchronizerActiveTimecodedInputSource*> UnreadySources;
		TArray<const FTimecodeSynchronizerActiveTimecodedInputSource*> InvalidSources;

		const FFrameRate FrameRate = GetFrameRate();
		for (FTimecodeSynchronizerActiveTimecodedInputSource& InputSource : SynchronizedSources)
		{
			InputSource.UpdateSourceState(FrameRate);
			if (!InputSource.IsInputSourceValid())
			{
				InvalidSources.Add(&InputSource);
			}
			else if (!InputSource.IsReady())
			{
				UnreadySources.Add(&InputSource);
			}
		}

		// Don't track readiness for these sources, they are not actively being used.
		for (FTimecodeSynchronizerActiveTimecodedInputSource& InputSource : NonSynchronizedSources)
		{
			InputSource.UpdateSourceState(FrameRate);
			if (!InputSource.IsInputSourceValid())
			{
				InvalidSources.Add(&InputSource);
			}
		}

		const FString StateString = SynchronizationStateToString(State);
		if (InvalidSources.Num() > 0)
		{
			for (const FTimecodeSynchronizerActiveTimecodedInputSource* InvalidSource : InvalidSources)
			{
				UE_LOGF(LogTimecodeSynchronizer, Error, "Source '%ls' is not valid during State '%ls'", *(InvalidSource->GetDisplayName()), *StateString);
			}
		}

		// Process our unready sources.
		// This is done here to keep the loops above fairly clean.
		if (bTreatUnreadyAsError && UnreadySources.Num() > 0)
		{
			for (const FTimecodeSynchronizerActiveTimecodedInputSource* UnreadySource : UnreadySources)
			{
				UE_LOGF(LogTimecodeSynchronizer, Error, "Source '%ls' became unready during State '%ls'", *(UnreadySource->GetDisplayName()), *StateString);
			}
		}

		if (InvalidSources.Num() > 0 || (bTreatUnreadyAsError && UnreadySources.Num() > 0))
		{
			SwitchState(ESynchronizationState::Error);
		}
	}
}

FFrameTime UTimecodeSynchronizer::CalculateSyncTime()
{
	if (CachedSyncState.RolloverFrame.IsSet())
	{
		return UTimeSynchronizationSource::AddOffsetWithRolloverModulus(CurrentProviderFrameTime, ActualFrameOffset, CachedSyncState.RolloverFrame.GetValue());
	}
	else
	{
		return CurrentProviderFrameTime + ActualFrameOffset;
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#undef LOCTEXT_NAMESPACE
