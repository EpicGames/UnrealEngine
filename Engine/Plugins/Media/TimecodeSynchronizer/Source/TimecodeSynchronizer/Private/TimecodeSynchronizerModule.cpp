// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimecodeSynchronizerModule.h"

#include "TimecodeSynchronizer.h"
#include "TimecodeSynchronizerProjectSettings.h"


DEFINE_LOG_CATEGORY(LogTimecodeSynchronizer)

PRAGMA_DISABLE_DEPRECATION_WARNINGS

//////////////////////////////////////////////////////////////////////////
// FTimecodeSynchronizer
class FTimecodeSynchronizer : public ITimecodeSynchronizerModule, public FSelfRegisteringExec
{
public:
	//~ Begin FSelfRegisteringExec Interface.
	virtual bool Exec(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		if (FParse::Command(&Cmd, TEXT("TimecodeSynchronizer")))
		{
			if (FParse::Command(&Cmd, TEXT("Synchronization")))
			{
				if (FParse::Command(&Cmd, TEXT("Start")))
				{
					return HandleStartSynchronization();
				}
				else if (FParse::Command(&Cmd, TEXT("Stop")))
				{
					return HandleStopSynchronization();
				}
				return true;
			}
		}
		return false;
	}
	//~ End FSelfRegisteringExec Interface.
	
	bool HandleStartSynchronization()
	{
		TSoftObjectPtr<UTimecodeSynchronizer>& MediaAsset = UTimecodeSynchronizerProjectSettings::StaticClass()->GetDefaultObject<UTimecodeSynchronizerProjectSettings>()->DefaultTimecodeSynchronizer;
		if (!MediaAsset.IsNull())
		{
			if (UTimecodeSynchronizer* Asset = MediaAsset.LoadSynchronous())
			{
				Asset->StartSynchronization();
			}
		}
		return true;
	}
	
	bool HandleStopSynchronization()
	{
		TSoftObjectPtr<UTimecodeSynchronizer>& MediaAsset = UTimecodeSynchronizerProjectSettings::StaticClass()->GetDefaultObject<UTimecodeSynchronizerProjectSettings>()->DefaultTimecodeSynchronizer;
		if (!MediaAsset.IsNull())
		{
			if (UTimecodeSynchronizer* Asset = MediaAsset.LoadSynchronous())
			{
				Asset->StopSynchronization();
			}
		}
		return true;
	}
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FTimecodeSynchronizer, TimecodeSynchronizer);

PRAGMA_ENABLE_DEPRECATION_WARNINGS
