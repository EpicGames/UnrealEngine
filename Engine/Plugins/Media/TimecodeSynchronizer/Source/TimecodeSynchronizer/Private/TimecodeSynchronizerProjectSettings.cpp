// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimecodeSynchronizerProjectSettings.h"
