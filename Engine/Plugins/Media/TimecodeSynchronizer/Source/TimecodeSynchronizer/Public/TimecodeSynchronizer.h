// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/TimecodeProvider.h"

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Tickable.h"
#include "TimeSynchronizationSource.h"

#include "Misc/QualifiedFrameTime.h"

#include "TimecodeSynchronizer.generated.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

class UFixedFrameRateCustomTimeStep;

/**
 * Defines the various modes that the synchronizer can use to try and achieve synchronization.
 */
UENUM()
enum class UE_DEPRECATED(5.0, "The TimecodeSynchronizer plugin is deprecated. Please update your project to use the features of the TimedDataMonitor plugin.") ETimecodeSynchronizationSyncMode
{
	/**
	 * User will specify an offset (number of frames) from the Timecode Source (see ETimecodeSycnrhonizationTimecodeType).
	 * This offset may be positive or negative depending on the latency of the source.
	 * Synchronization will be achieved once the synchronizer detects all input sources have frames that correspond
	 * with the offset timecode.
	 *
	 * This is suitable for applications trying to keep multiple Unreal Engine instances in sync while using nDisplay / genlock.
	 */
	UserDefinedOffset,

	/**
	 * Engine will try and automatically determine an appropriate offset based on what frames are available
	 * on the given sources.
	 *
	 * This is suitable for running a single Unreal Engine instance that just wants to synchronize its inputs.
	 */
	Auto,

	/**
	 * The same as Auto except that instead of trying to find a suitable timecode nearest to the
	 * newest common frame, we try to find a suitable timecode nearest to the oldest common frame.
	 */
	AutoOldest,
};

/**
 * Enumerates Timecode source type.
 */
UENUM()
enum class UE_DEPRECATED(5.0, "The TimecodeSynchronizer plugin is deprecated. Please update your project to use the features of the TimedDataMonitor plugin.") ETimecodeSynchronizationTimecodeType
{
	/** Use the configured Engine Default Timecode provider. */
	DefaultProvider,

	/** Use an external Timecode Provider to provide the timecode to follow. */
	TimecodeProvider,

	/** Use one of the InputSource as the Timecode Provider. */
	InputSource
};

/**
 * Enumerates possible framerate source
 */
UENUM()
enum class UE_DEPRECATED(5.0, "The TimecodeSynchronizer plugin is deprecated. Please update your project to use the features of the TimedDataMonitor plugin.") ETimecodeSynchronizationFrameRateSources: uint8
{
	EngineCustomTimeStepFrameRate UMETA(ToolTip="Frame Rate of engine custom time step if it is of type UFixedFrameRateCustomTimeStep."),
	CustomFrameRate UMETA(ToolTip = "Custom Frame Rate selected by the user.")
};


/**
 * Enumerates Synchronization related events.
 */
enum class UE_DEPRECATED(5.0, "The TimecodeSynchronizer plugin is deprecated. Please update your project to use the features of the TimedDataMonitor plugin.") ETimecodeSynchronizationEvent
{
	/** The synchronization procedure has started. */
	SynchronizationStarted,

	/** The synchronization procedure failed. */
	SynchronizationFailed,

	/** The synchronization procedure succeeded. */
	SynchronizationSucceeded,

	/** Synchronization was stopped. Note, this won't be called if Synchronization failed. */
	SynchronizationStopped,
};

/** Cached values to use during synchronization / while synchronized */
struct UE_DEPRECATED(5.0, "The TimecodeSynchronizer plugin is deprecated. Please update your project to use the features of the TimedDataMonitor plugin.") FTimecodeSynchronizerCachedSyncState;
struct FTimecodeSynchronizerCachedSyncState
{
	/** If we're using rollover, the frame time that represents the rollover point (e.g., the modulus). */
	TOptional<FFrameTime> RolloverFrame;

	/** The FrameRate of the synchronizer. */
	FFrameRate FrameRate;

	/** Synchronization mode that's being used. */
	ETimecodeSynchronizationSyncMode SyncMode;

	/** Frame offset that will be used if SyncMode != Auto; */
	int32 FrameOffset;
};

/** Cached frame values for a given source. */
struct UE_DEPRECATED(5.0, "The TimecodeSynchronizer plugin is deprecated. Please update your project to use the features of the TimedDataMonitor plugin.") FTimecodeSourceState;
struct FTimecodeSourceState
{
	/** Frame time of the newest available sample. */
	FFrameTime NewestAvailableSample;

	/** Frame time of the oldest available sample. */
	FFrameTime OldestAvailableSample;
};

/**
 * Provides a wrapper around a UTimeSynchronizerSource, and caches data necessary
 * to provide synchronization.
 *
 * The values are typically updated once per frame.
 */
struct UE_DEPRECATED(5.0, "The TimecodeSynchronizer plugin is deprecated. Please update your project to use the features of the TimedDataMonitor plugin.") FTimecodeSynchronizerActiveTimecodedInputSource;
USTRUCT()
struct FTimecodeSynchronizerActiveTimecodedInputSource
{
	GENERATED_BODY()

public:

	FTimecodeSynchronizerActiveTimecodedInputSource()
		: bIsReady(false)
		, bCanBeSynchronized(false)
		, TotalNumberOfSamples(0)
		, FrameRate(60, 1)
		, InputSource(nullptr)
	{
	}

	FTimecodeSynchronizerActiveTimecodedInputSource(UTimeSynchronizationSource* Source)
		: bIsReady(false)
		, bCanBeSynchronized(Source->bUseForSynchronization)
		, TotalNumberOfSamples(0)
		, FrameRate(60, 1)
		, InputSource(Source)
	{
	}

	/** Updates the internal state of this source, returning whether or not the source is ready (e.g. IsReady() == true). */
	const bool UpdateSourceState(const FFrameRate& SynchronizerFrameRate);

	inline const UTimeSynchronizationSource* GetInputSource() const
	{
		return InputSource;
	}

	inline bool IsInputSourceValid() const
	{
		return nullptr != InputSource;
	}

	inline FString GetDisplayName() const
	{
		return InputSource->GetDisplayName();
	}

	/** Whether or not this source is ready. */
	inline bool IsReady() const
	{
		return bIsReady;
	}

	/** Whether or not this source can be synchronized. */
	inline bool CanBeSynchronized() const
	{
		return bCanBeSynchronized;
	}

	/** Gets the FrameRate of the source. */
	inline const FFrameRate& GetFrameRate() const
	{
		return FrameRate;
	}

	/** Gets the state of the Source relative to its own frame rate. */
	inline const FTimecodeSourceState& GetInputSourceState() const
	{
		return InputSourceState;
	}

	/** Gets the state of the Source relative to the Synchronizer's frame rate. */
	inline const FTimecodeSourceState& GetSynchronizerRelativeState() const
	{
		return SynchronizerRelativeState;
	}

private:

	/* Flag stating if the source is ready */
	UPROPERTY(VisibleAnywhere, Transient, Category=Debug, Meta=(DisplayName = "Is Ready"))
	bool bIsReady;

	/* Flag stating if this source can be synchronized */
	UPROPERTY(VisibleAnywhere, Transient, Category=Debug, Meta=(DisplayName = "Can Be Synchronized"))
	bool bCanBeSynchronized;

	UPROPERTY(VisibleAnywhere, Transient, Category=Debug)
	int32 TotalNumberOfSamples;

	FFrameRate FrameRate;

	FTimecodeSourceState InputSourceState;
	FTimecodeSourceState SynchronizerRelativeState;

	/* Associated source pointers */
	UPROPERTY(VisibleAnywhere, Transient, Category = Debug, Meta=(DisplayName="Input Source"))
	TObjectPtr<UTimeSynchronizationSource> InputSource;

	FTimecodeSynchronizerActiveTimecodedInputSource(const FTimecodeSynchronizerActiveTimecodedInputSource&) = delete;
	FTimecodeSynchronizerActiveTimecodedInputSource& operator=(const FTimecodeSynchronizerActiveTimecodedInputSource&) = delete;
	FTimecodeSynchronizerActiveTimecodedInputSource(FTimecodeSynchronizerActiveTimecodedInputSource&&) = delete;
	FTimecodeSynchronizerActiveTimecodedInputSource& operator=(FTimecodeSynchronizerActiveTimecodedInputSource&&) = delete;
};

template<> struct UE_DEPRECATED(5.0, "The TimecodeSynchronizer plugin is deprecated. Please update your project to use the features of the TimedDataMonitor plugin.") TStructOpsTypeTraits<FTimecodeSynchronizerActiveTimecodedInputSource>;

template<>
struct TStructOpsTypeTraits<FTimecodeSynchronizerActiveTimecodedInputSource> : public TStructOpsTypeTraitsBase2<FTimecodeSynchronizerActiveTimecodedInputSource>
{
	enum
	{
		WithCopy = false
	};
};

/**
 * Timecode Synchronizer is intended to correlate multiple timecode sources to help ensure
 * that all sources can produce data that is frame aligned.
 *
 * This typically works by having sources buffer data until we have enough frames that
 * such that we can find an overlap. Once that process is finished, the Synchronizer will
 * provide the appropriate timecode to the engine (which can be retrieved via FApp::GetTimecode
 * and FApp::GetTimecodeFrameRate).
 *
 * Note, the Synchronizer doesn't perform any buffering of data itself (that is left up to
 * TimeSynchronizationSources). Instead, the synchronizer simply acts as a coordinator
 * making sure all sources are ready, determining if sync is possible, etc.
 */
class UE_DEPRECATED(5.0, "The TimecodeSynchronizer plugin is deprecated. Please update your project to use the features of the TimedDataMonitor plugin.") UTimecodeSynchronizer;
UCLASS()
class TIMECODESYNCHRONIZER_API UTimecodeSynchronizer : public UTimecodeProvider, public FTickableGameObject
{
	GENERATED_BODY()

public:

	UTimecodeSynchronizer();

	//~ Begin UObject Interface
	virtual void BeginDestroy() override;
#if WITH_EDITOR
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface

	//~ Begin TimecodeProvider Interface
	FQualifiedFrameTime GetQualifiedFrameTime() const override;
	virtual ETimecodeProviderSynchronizationState GetSynchronizationState() const override;
	virtual bool Initialize(class UEngine* InEngine) override;
	virtual void Shutdown(class UEngine* InEngine) override;
	//~ End TimecodeProvider Interface

	//~ Begin FTickableGameObject implementation
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Conditional; }
	virtual bool IsTickableWhenPaused() const override { return true; }
	virtual bool IsTickableInEditor() const override { return true; }
	virtual bool IsTickable() const override;
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(TimecodeSynchronizer, STATGROUP_Tickables); }
	//~ End FTickableGameObject

public:

	/**
	 * Starts the synchronization process. Does nothing if we're already synchronized, or attempting to synchronize.
	 *
	 * @return True if the synchronization process was successfully started (or was previously started).
	 */
	bool StartSynchronization();

	/** Stops the synchronization process. Does nothing if we're not synchronized, or attempting to synchronize. */
	void StopSynchronization();

	UE_DEPRECATED(4.21, "Please use GetSynchronizedSources.")
	const TArray<FTimecodeSynchronizerActiveTimecodedInputSource>& GetTimecodedSources() const { return GetSynchronizedSources(); }

	UE_DEPRECATED(4.21, "Please use GetNonSynchronizedSources.")
	const TArray<FTimecodeSynchronizerActiveTimecodedInputSource>& GetSynchronizationSources() const { return GetNonSynchronizedSources(); }

	/** Returns the list of sources that are used to perform synchronization. */
	const TArray<FTimecodeSynchronizerActiveTimecodedInputSource>& GetSynchronizedSources() const { return SynchronizedSources; }

	/** Returns the list of sources that are not actively being used in synchronization. */
	const TArray<FTimecodeSynchronizerActiveTimecodedInputSource>& GetNonSynchronizedSources() const { return NonSynchronizedSources; }
	
	/** Returns the index of the Main Synchronization Source in the Synchronized Sources list. */
	int32 GetActiveMainSynchronizationTimecodedSourceIndex() const { return MainSynchronizationSourceIndex; }

	/**
	 * Get an event delegate that is invoked when a Asset synchronization event occurred.
	 *
	 * @return The delegate.
	 */
	DECLARE_EVENT_OneParam(UTimecodeSynchronizer, FOnTimecodeSynchronizationEvent, ETimecodeSynchronizationEvent /*Event*/)
	FOnTimecodeSynchronizationEvent& OnSynchronizationEvent()
	{
		return SynchronizationEvent;
	}

	/**
	 * Adds a "runtime" source to the synchronizer.
	 * These sources can only be added from StartSynchronization callback, and will automatically
	 * be removed once synchronization has stopped (or failed).
	 *
	 * While the synchronization process is active, it's guaranteed that the TimecodeSynchronizer
	 * will keep a hard reference to the source.
	 *
	 * Sources added this way should be added every time the StartSynchronization event is fired.
	 */
	void AddRuntimeTimeSynchronizationSource(UTimeSynchronizationSource* Source);

	/**
	 * Get Current System Frame Time which is synchronized
	 *
	 * @return FrameTime
	 */
	FFrameTime GetCurrentSystemFrameTime() const
	{
		if (CurrentSystemFrameTime.IsSet())
		{
			return CurrentSystemFrameTime.GetValue();
		}
		else
		{
			return FFrameTime();
		}
	}


private:

	/** Synchronization states */
	enum class ESynchronizationState : uint8
	{
		None,
		Error,
		Initializing,								// Kicking off the initialization process.
		PreRolling_WaitGenlockTimecodeProvider,		// wait for the TimecodeProvider & CustomTimeStep to be Ready
		PreRolling_WaitReadiness,					// wait for all source to be Ready
		PreRolling_Synchronizing,					// wait and find a valid Timecode to start with
		Synchronized,								// all sources are running and synchronized
	};

	inline static FString SynchronizationStateToString(ESynchronizationState InState)
	{
		switch (InState)
		{
		case ESynchronizationState::None:
			return FString(TEXT("None"));

		case ESynchronizationState::Initializing:
			return FString(TEXT("Initializing"));

		case ESynchronizationState::PreRolling_WaitGenlockTimecodeProvider:
			return FString(TEXT("WaitGenlockTimecodeProvider"));

		case ESynchronizationState::PreRolling_WaitReadiness:
			return FString(TEXT("WaitReadiness"));

		case ESynchronizationState::PreRolling_Synchronizing:
			return FString(TEXT("Synchronizing"));

		case ESynchronizationState::Synchronized:
			return FString(TEXT("Synchronized"));

		case ESynchronizationState::Error:
			return FString(TEXT("Error"));

		default:
			return FString::Printf(TEXT("Invalid State %d"), static_cast<int32>(InState));
		}
	}

	/** Callback when the engine's TimecodeProvider changed. */
	void OnTimecodeProviderChanged();

	/** Registers asset to MediaModule tick */
	void SetTickEnabled(bool bEnabled);

	/** Switches on current state and ticks it */
	void Tick_Switch();

	bool ShouldTick();

	/** Test if the genlock & timecode provider are properly setup */
	bool Tick_TestGenlock();
	bool Tick_TestTimecode();

	/** Process PreRolling_WaitGenlockTimecodeProvider state. */
	void TickPreRolling_WaitGenlockTimecodeProvider();

	/** Process PreRolling_WaitReadiness state. */
	void TickPreRolling_WaitReadiness();
	
	/** Process PreRolling_Synchronizing state. */
	void TickPreRolling_Synchronizing();

	/** Process Synchronized state. */
	void Tick_Synchronized();

	/** Register TimecodeSynchronizer as the TimecodeProvider */
	void Register();
	
	/** Unregister TimecodeSynchronizer as the TimecodeProvider */
	void Unregister();

	void OpenSources();
	void StartSources();
	void CloseSources();

	/** Updates and caches the state of the sources. */
	void UpdateSourceStates();
	FFrameTime CalculateSyncTime();

	FTimecode GetTimecodeInternal() const;
	FFrameRate GetFrameRateInternal() const;

	bool IsSynchronizing() const;
	bool IsSynchronized() const;
	bool IsError() const;

	/** Changes internal state and execute it if required */
	void SwitchState(const ESynchronizationState NewState);

	FFrameTime GetProviderFrameTime() const;

public:

	/** Frame Rate Source. */
	UPROPERTY(EditAnywhere, Category = "Frame Rate Settings", Meta = (DisplayName = "Frame Rate Source"))
	ETimecodeSynchronizationFrameRateSources FrameRateSource;

	/** The fixed framerate to use. */
	UPROPERTY(EditAnywhere, Category="Frame Rate Settings", Meta=(ClampMin="15.0"))
	FFrameRate FixedFrameRate;

public:
	/** Use a Timecode Provider. */
	UPROPERTY(EditAnywhere, Category="Timecode Provider", Meta=(DisplayName="Select"))
	ETimecodeSynchronizationTimecodeType TimecodeProviderType;

	/** Custom strategy to tick in a interval. */
	UPROPERTY(EditAnywhere, Instanced, Category="Timecode Provider", Meta=(DisplayName="Timecode Source"))
	TObjectPtr<UTimecodeProvider> TimecodeProvider;

	/**
	 * Index of the source that drives the synchronized Timecode.
	 * The source need to be timecoded and flag as bUseForSynchronization
	 */
	UPROPERTY(EditAnywhere, Category="Timecode Provider")
	int32 MainSynchronizationSourceIndex;

public:
	/** Enable verification of margin between synchronized time and source time */
	UPROPERTY()
	bool bUsePreRollingTimecodeMarginOfErrors;

	/** Maximum gap size between synchronized time and source time */
	UPROPERTY(EditAnywhere, Category="Synchronization", Meta=(EditCondition="bUsePreRollingTimecodeMarginOfErrors", ClampMin="0"))
	int32 PreRollingTimecodeMarginOfErrors;

	/** Enable PreRoll timeout */
	UPROPERTY()
	bool bUsePreRollingTimeout;

	/** How long to wait for all source to be ready */
	UPROPERTY(EditAnywhere, Category="Synchronization", Meta=(EditCondition="bUsePreRollingTimeout", ClampMin="0.0", ForceUnits=s))
	float PreRollingTimeout; 

public:

	//! ONLY MODIFY THESE IN EDITOR
	//! TODO: Deprecate this and make it private.
	UPROPERTY(EditAnywhere, Instanced, Category="Input")
	TArray<TObjectPtr<UTimeSynchronizationSource>> TimeSynchronizationInputSources;

private:

	UPROPERTY(Transient)
	TArray<TObjectPtr<UTimeSynchronizationSource>> DynamicSources;

	/** What mode will be used for synchronization. */
	UPROPERTY(EditAnywhere, Category = "Synchronization")
	ETimecodeSynchronizationSyncMode SyncMode;

	/**
	 * When UserDefined mode is used, the number of frames delayed from the Provider's timecode.
	 * Negative values indicate the used timecode will be ahead of the Provider's.
	 */
	UPROPERTY(EditAnywhere, Category = "Synchronization", Meta=(ClampMin="-640", ClampMax="640"))
	int32 FrameOffset;

	/**
	 * Similar to FrameOffset.
	 * For Auto mode, this represents the number of frames behind the newest synced frame.
	 * For AutoModeOldest, the is the of frames ahead of the last synced frame.
	 */
	UPROPERTY(EditAnywhere, Category = "Synchronization", Meta = (ClampMin = "0", ClampMax = "640"))
	int32 AutoFrameOffset = 3;

	/** Whether or not the specified Provider's timecode rolls over. (Rollover is expected to occur at Timecode 24:00:00:00). */
	UPROPERTY(EditAnywhere, Category = "Synchronization")
	bool bWithRollover = false;

	/** Sources used for synchronization */
	UPROPERTY(Transient, DuplicateTransient, VisibleAnywhere, Category=Debug)
	TArray<FTimecodeSynchronizerActiveTimecodedInputSource> SynchronizedSources;

	/* Sources that wants to be synchronized */
	UPROPERTY(Transient, DuplicateTransient, VisibleAnywhere, Category=Debug)
	TArray<FTimecodeSynchronizerActiveTimecodedInputSource> NonSynchronizedSources;

	UPROPERTY(Transient)
	TObjectPtr<UFixedFrameRateCustomTimeStep> RegisteredCustomTimeStep;

	UPROPERTY(Transient)
	TObjectPtr<UTimecodeProvider> CachedPreviousTimecodeProvider;

	UPROPERTY(Transient)
	TObjectPtr<UTimecodeProvider> CachedProxiedTimecodeProvider;

	UPROPERTY(Transient, DuplicateTransient, VisibleAnywhere, Category = "Synchronization")
	int32 ActualFrameOffset;

private:

	bool bIsTickEnabled;

	int64 LastUpdatedSources = 0;

	/** The actual synchronization state */
	ESynchronizationState State;
	
	/** Frame time that we'll use for the system */
	TOptional<FFrameTime> CurrentSystemFrameTime;

	/** The current frame from our specified provider. */
	FFrameTime CurrentProviderFrameTime;
	
	/** Timestamp when PreRolling has started */
	double StartPreRollingTime;
	
	/** Whether or not we are registered as the TimecodeProvider */
	bool bRegistered;
	float PreviousFixedFrameRate;
	bool bPreviousUseFixedFrameRate;
	
	/** Index of the active source that drives the synchronized Timecode*/
	int32 ActiveMainSynchronizationTimecodedSourceIndex;

	/** An event delegate that is invoked when a synchronization event occurred. */
	FOnTimecodeSynchronizationEvent SynchronizationEvent;

	FTimecodeSynchronizerCachedSyncState CachedSyncState;

	bool bFailGuard;
	bool bAddSourcesGuard;
	bool bShouldResetTimecodeProvider;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
