// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define TIMECODESYNCHRONIZER_MODULE_NAME TEXT("TimecodeSynchronizer")

DECLARE_LOG_CATEGORY_EXTERN(LogTimecodeSynchronizer, Verbose, All);

class UE_DEPRECATED(5.0, "The TimecodeSynchronizer plugin is deprecated. Please update your project to use the features of the TimedDataMonitor plugin.") ITimecodeSynchronizerModule;
class TIMECODESYNCHRONIZER_API ITimecodeSynchronizerModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to ITimecodeSynchronizerModule
	 * @return Returns TimecodeSynchronizerModule singleton instance, loading the module on demand if needed
	 */
	static inline ITimecodeSynchronizerModule& Get()
	{
		return FModuleManager::LoadModuleChecked<ITimecodeSynchronizerModule>(TIMECODESYNCHRONIZER_MODULE_NAME);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(TIMECODESYNCHRONIZER_MODULE_NAME);
	}
};

#undef TIMECODESYNCHRONIZER_MODULE_NAME
