// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "TimecodeSynchronizer.h"

#include "TimecodeSynchronizerProjectSettings.generated.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
/**
 * Global settings for TimecodeSynchronizer
 */
class UE_DEPRECATED(5.0, "The TimecodeSynchronizer plugin is deprecated. Please update your project to use the features of the TimedDataMonitor plugin.") UTimecodeSynchronizerProjectSettings;
UCLASS(config=Engine)
class TIMECODESYNCHRONIZER_API UTimecodeSynchronizerProjectSettings : public UObject
{
public:
	GENERATED_BODY()

	UTimecodeSynchronizerProjectSettings()
		: bDisplayInToolbar(false)
	{ }

public:
	/** Display the timecode synchronizer icon in the editor toolbar. */
	UPROPERTY(Config, EditAnywhere, Category="TimecodeSynchronizer", meta=(ConfigRestartRequired=true))
	bool bDisplayInToolbar;

	UPROPERTY(config, EditAnywhere, Category="TimecodeSynchronizer")
	TSoftObjectPtr<UTimecodeSynchronizer> DefaultTimecodeSynchronizer;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
	{
		if (GET_MEMBER_NAME_CHECKED(ThisClass, DefaultTimecodeSynchronizer) == PropertyChangedEvent.GetPropertyName())
		{
			OnDefaultTimecodeSynchronizerChanged.Broadcast();
		}
		
		Super::PostEditChangeProperty(PropertyChangedEvent);
	}
#endif

	FSimpleMulticastDelegate OnDefaultTimecodeSynchronizerChanged;
};

/**
 * Editor settings for TimecodeSynchronizer
 */
class UE_DEPRECATED(5.0, "The TimecodeSynchronizer plugin is deprecated. Please update your project to use the features of the TimedDataMonitor plugin.") UTimecodeSynchronizerEditorSettings;
UCLASS(config= EditorPerProjectUserSettings)
class TIMECODESYNCHRONIZER_API UTimecodeSynchronizerEditorSettings : public UObject
{
public:
	GENERATED_BODY()

public:

	UPROPERTY(config, EditAnywhere, Category="TimecodeSynchronizer")
	TSoftObjectPtr<UTimecodeSynchronizer> UserTimecodeSynchronizer;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS