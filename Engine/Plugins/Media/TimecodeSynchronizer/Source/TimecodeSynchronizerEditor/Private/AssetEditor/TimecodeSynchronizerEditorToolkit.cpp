// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetEditor/TimecodeSynchronizerEditorToolkit.h"

#include "DetailsViewArgs.h"
#include "TimecodeSynchronizer.h"
#include "Framework/Commands/UICommandList.h"
#include "UI/TimecodeSynchronizerEditorStyle.h"
#include "Subsystems/ImportSubsystem.h"
#include "Widgets/STimecodeSynchronizerSourceViewer.h"
#include "Widgets/STimecodeSynchronizerWidget.h"

#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Notifications/SProgressBar.h"
 
#define LOCTEXT_NAMESPACE "TimecodeSynchronizerEditor"

namespace TimecodeSynchronizerEditorToolkit
{
	const FName AppIdentifier = TEXT("TimecodeSynchronizerEditorApp");
	const FName PropertiesTabId(TEXT("TimecodeSynchronizerEditor_Properties"));
	const FName SourceViewerTabId(TEXT("TimecodeSynchronizerEditor_SourceViewer"));
	const FName SynchronizerWidgetTabId(TEXT("TimecodeSynchronizerEditor_SynchronizerWidget"));
	const FName Layout(TEXT("Standalone_TimecodeSynchronizerEditor_Layout_v1"));
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

TSharedRef<FTimecodeSynchronizerEditorToolkit> FTimecodeSynchronizerEditorToolkit::CreateEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class UTimecodeSynchronizer* InTimecodeSynchronizer)
{
	TSharedRef<FTimecodeSynchronizerEditorToolkit> NewEditor(new FTimecodeSynchronizerEditorToolkit());
	NewEditor->InitTimecodeSynchronizerEditor(Mode, InitToolkitHost, InTimecodeSynchronizer);
	return NewEditor;
}

void FTimecodeSynchronizerEditorToolkit::InitTimecodeSynchronizerEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UTimecodeSynchronizer* InTimecodeSynchronizer)
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.AddRaw(this, &FTimecodeSynchronizerEditorToolkit::HandleAssetPostImport);

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout(TimecodeSynchronizerEditorToolkit::Layout)
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.9f)
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.5f)
					->Split
					(
						// Source display
						FTabManager::NewStack()
						->SetHideTabWell(true)
						->AddTab(TimecodeSynchronizerEditorToolkit::SourceViewerTabId, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewSplitter()
						->SetSizeCoefficient(0.4f)
						->Split
						(
							FTabManager::NewStack()
							->AddTab(TimecodeSynchronizerEditorToolkit::PropertiesTabId, ETabState::OpenedTab)
						)
					)
				)
			)
		);

	InTimecodeSynchronizer->OnSynchronizationEvent().AddSP(this, &FTimecodeSynchronizerEditorToolkit::HandleSynchronizationEvent);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	Super::InitAssetEditor(Mode, InitToolkitHost, TimecodeSynchronizerEditorToolkit::AppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, InTimecodeSynchronizer);

	ExtendToolBar();

	// Get the list of objects to edit the details of
	TArray<UObject*> ObjectsToEditInDetailsView;
	ObjectsToEditInDetailsView.Add(InTimecodeSynchronizer);

	// Ensure all objects are transactable for undo/redo in the details panel
	for (UObject* ObjectToEditInDetailsView : ObjectsToEditInDetailsView)
	{
		ObjectToEditInDetailsView->SetFlags(RF_Transactional);
	}

	if (DetailsView.IsValid())
	{
		// Make sure details window is pointing to our object
		DetailsView->SetObjects(ObjectsToEditInDetailsView);
		DetailsView->SetIsPropertyEditingEnabledDelegate
			(
				FIsPropertyEditingEnabled::CreateLambda([&]
				{
					if (UTimecodeSynchronizer* Asset = GetTimecodeSynchronizer())
					{
						ETimecodeProviderSynchronizationState State = Asset->GetSynchronizationState();
						return State == ETimecodeProviderSynchronizationState::Closed;
					}
					return false;
				})
			);
	}
}

FTimecodeSynchronizerEditorToolkit::~FTimecodeSynchronizerEditorToolkit()
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.RemoveAll(this);
	if (UTimecodeSynchronizer* Asset = GetTimecodeSynchronizer())
	{
		Asset->OnSynchronizationEvent().RemoveAll(this);
	}
}

FName FTimecodeSynchronizerEditorToolkit::GetToolkitFName() const
{
	return TimecodeSynchronizerEditorToolkit::AppIdentifier;
}

FText FTimecodeSynchronizerEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Timecode Synchronizer Editor");
}

FText FTimecodeSynchronizerEditorToolkit::GetToolkitName() const
{
	const UObject* EditingObject = GetEditingObject();

	const bool bDirtyState = EditingObject->GetOutermost()->IsDirty();

	FFormatNamedArguments Args;
	Args.Add(TEXT("ToolkitName"), GetBaseToolkitName());
	Args.Add(TEXT("ObjectName"), FText::FromString(EditingObject->GetName()));
	Args.Add(TEXT("DirtyState"), bDirtyState ? FText::FromString(TEXT("*")) : FText::GetEmpty());
	return FText::Format(LOCTEXT("ToolkitTitle", "{ObjectName}{DirtyState} - {ToolkitName}"), Args);
}

FString FTimecodeSynchronizerEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "TimecodeSynchronizer ").ToString();
}

FLinearColor FTimecodeSynchronizerEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.0f, 0.0f, 0.2f, 0.5f);
}

void FTimecodeSynchronizerEditorToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_TimecodeSynchronizerEditor", "Timecode Synchronizer Editor"));

	Super::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(TimecodeSynchronizerEditorToolkit::PropertiesTabId, FOnSpawnTab::CreateSP(this, &FTimecodeSynchronizerEditorToolkit::SpawnPropertiesTab))
		.SetDisplayName(LOCTEXT("PropertiesTab", "Details"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
	
	InTabManager->RegisterTabSpawner(TimecodeSynchronizerEditorToolkit::SourceViewerTabId, FOnSpawnTab::CreateSP(this, &FTimecodeSynchronizerEditorToolkit::SpawnSourceViewerTab))
		.SetDisplayName(LOCTEXT("SourceViewerTab", "Sources"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	InTabManager->RegisterTabSpawner(TimecodeSynchronizerEditorToolkit::SynchronizerWidgetTabId, FOnSpawnTab::CreateSP(this, &FTimecodeSynchronizerEditorToolkit::SpawnSynchronizerWidgetTab))
		.SetDisplayName(LOCTEXT("SynchronizationTab", "Synchronization"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FTimecodeSynchronizerEditorStyle::GetStyleSetName(), "SynchronizationWidget.small"));
}

void FTimecodeSynchronizerEditorToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	Super::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(TimecodeSynchronizerEditorToolkit::PropertiesTabId);
	InTabManager->UnregisterTabSpawner(TimecodeSynchronizerEditorToolkit::SourceViewerTabId);
}

UTimecodeSynchronizer* FTimecodeSynchronizerEditorToolkit::GetTimecodeSynchronizer() const
{
	return Cast<UTimecodeSynchronizer>(GetEditingObject());
}

TSharedRef<SDockTab> FTimecodeSynchronizerEditorToolkit::SpawnPropertiesTab(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == TimecodeSynchronizerEditorToolkit::PropertiesTabId);

	return SNew(SDockTab)
		.Label(LOCTEXT("GenericDetailsTitle", "Details"))
		.TabColorScale(GetTabColorScale())
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 0.0f)
			.HAlign(HAlign_Fill)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SProgressBar)
					.ToolTipText(LOCTEXT("BufferingTooltip", "Buffering..."))
					.Visibility_Lambda([this]() -> EVisibility
					{
						if (UTimecodeSynchronizer* Asset = GetTimecodeSynchronizer())
						{
							ETimecodeProviderSynchronizationState State = Asset->GetSynchronizationState();
							return State == ETimecodeProviderSynchronizationState::Synchronizing ? EVisibility::Visible : EVisibility::Hidden;
						}
						return EVisibility::Hidden;
					})
				]
				+ SOverlay::Slot()
				[
					SNew(SColorBlock)
					.Color(this, &FTimecodeSynchronizerEditorToolkit::GetProgressColor)
					.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
					.Visibility_Lambda([this]() -> EVisibility
					{
						if (UTimecodeSynchronizer* Asset = GetTimecodeSynchronizer())
						{
							ETimecodeProviderSynchronizationState State = Asset->GetSynchronizationState();
							return State == ETimecodeProviderSynchronizationState::Synchronizing ? EVisibility::Hidden : EVisibility::Visible;
						}
						return EVisibility::Hidden;
					})
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0.0f, 0.0f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				DetailsView.ToSharedRef()
			]
		];
}

TSharedRef<SDockTab> FTimecodeSynchronizerEditorToolkit::SpawnSourceViewerTab(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == TimecodeSynchronizerEditorToolkit::SourceViewerTabId);

	TSharedPtr<SWidget> TabWidget = SNew(STimecodeSynchronizerSourceViewer, *GetTimecodeSynchronizer());

	return SNew(SDockTab)
		.Label(LOCTEXT("GenericSourceViewerTitle", "Sources"))
		.TabColorScale(GetTabColorScale())
		[
			TabWidget.ToSharedRef()
		];
}

TSharedRef<SDockTab> FTimecodeSynchronizerEditorToolkit::SpawnSynchronizerWidgetTab(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == TimecodeSynchronizerEditorToolkit::SynchronizerWidgetTabId);

	TSharedPtr<SWidget> TabWidget = SNew(STimecodeSynchronizerWidget, *GetTimecodeSynchronizer());

	return SNew(SDockTab)
		.Label(LOCTEXT("SynchronizerWidget", "Synchronization"))
		.TabColorScale(GetTabColorScale())
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot().AutoHeight()
			[
				TabWidget.ToSharedRef()
			]
		];
}

void FTimecodeSynchronizerEditorToolkit::HandleAssetPostImport(UFactory* InFactory, UObject* InObject)
{
	if (GetEditingObject() == InObject)
	{
		// The details panel likely needs to be refreshed if an asset was imported again
		TArray<UObject*> PostImportedEditingObjects;
		PostImportedEditingObjects.Add(InObject);
		DetailsView->SetObjects(PostImportedEditingObjects);
	}
}

void FTimecodeSynchronizerEditorToolkit::ExtendToolBar()
{
	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateLambda([&](FToolBarBuilder& ToolbarBuilder)
		{
			ToolbarBuilder.BeginSection("TimecodeSynchronizer");
			{
				ToolbarBuilder.AddToolBarButton(
					FUIAction(
						FExecuteAction::CreateLambda([&]()
						{
							if (UTimecodeSynchronizer* Asset = GetTimecodeSynchronizer())
							{
								Asset->StartSynchronization();
							}
						}),
						FCanExecuteAction::CreateLambda([&]()
						{
							if (UTimecodeSynchronizer* Asset = GetTimecodeSynchronizer())
							{
								ETimecodeProviderSynchronizationState State = Asset->GetSynchronizationState();
								return State == ETimecodeProviderSynchronizationState::Closed || State == ETimecodeProviderSynchronizationState::Error;
							}
							return false;
						}),
						FIsActionChecked()
					),
					NAME_None,
					LOCTEXT("PreRoll", "Start Synchronization"),
					LOCTEXT("PreRoll_ToolTip", "Start all medias and synchronize them."),
					FSlateIcon(FTimecodeSynchronizerEditorStyle::GetStyleSetName(), "Synchronized")
					);

				ToolbarBuilder.AddToolBarButton(
					FUIAction(
						FExecuteAction::CreateLambda([&]()
						{
							if (UTimecodeSynchronizer* Asset = GetTimecodeSynchronizer())
							{
								Asset->StopSynchronization();
							}
						}),
						FCanExecuteAction::CreateLambda([&]()
						{
							if (UTimecodeSynchronizer* Asset = GetTimecodeSynchronizer())
							{
								ETimecodeProviderSynchronizationState State = Asset->GetSynchronizationState();
								return State == ETimecodeProviderSynchronizationState::Synchronizing || State == ETimecodeProviderSynchronizationState::Synchronized;
							}
							return false;
						}),
						FIsActionChecked()
					),
					NAME_None,
					LOCTEXT("StopPreRoll", "Stop Synchronization"),
					LOCTEXT("StopPreRoll_ToolTip", "Stop all medias and remove the genlock (if enabled)."),
					FSlateIcon(FTimecodeSynchronizerEditorStyle::GetStyleSetName(), "Stop")
					);
			}
			ToolbarBuilder.EndSection();
		})
	);
	AddToolbarExtender(ToolbarExtender);

	RegenerateMenusAndToolbars();
}

FLinearColor FTimecodeSynchronizerEditorToolkit::GetProgressColor() const
{
	if (UTimecodeSynchronizer* Asset = GetTimecodeSynchronizer())
	{
		ETimecodeProviderSynchronizationState State = Asset->GetSynchronizationState();
		switch(State)
		{
			case ETimecodeProviderSynchronizationState::Error:
				return FColor::Red;
			case ETimecodeProviderSynchronizationState::Closed:
				return FColor::Black;
			case ETimecodeProviderSynchronizationState::Synchronized:
				return FColor::Green;
			case ETimecodeProviderSynchronizationState::Synchronizing:
				return FColor::Yellow;
		}
	}
	return FColor::Black;
}

void FTimecodeSynchronizerEditorToolkit::HandleSynchronizationEvent(ETimecodeSynchronizationEvent Event)
{
	if (Event == ETimecodeSynchronizationEvent::SynchronizationFailed)
	{
		if (UTimecodeSynchronizer* Asset = GetTimecodeSynchronizer())
		{
			FNotificationInfo NotificationInfo(LOCTEXT("FailedError", "Failed to synchronize. Check Output Log for details!"));

			NotificationInfo.ExpireDuration = 2.0f;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo)->SetCompletionState(SNotificationItem::CS_Fail);
		}
	}
}

void FTimecodeSynchronizerEditorToolkit::RemoveEditingObject(UObject* Object)
{
	if (UTimecodeSynchronizer* Asset = GetTimecodeSynchronizer())
	{
		if (Asset == Object)
		{
			Asset->OnSynchronizationEvent().RemoveAll(this);
		}
	}
	Super::RemoveEditingObject(Object);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
