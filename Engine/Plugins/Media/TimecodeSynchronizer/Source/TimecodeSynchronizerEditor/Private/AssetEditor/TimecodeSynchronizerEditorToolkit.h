// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Toolkits/AssetEditorToolkit.h"

class FSpawnTabArgs;
class UTimecodeSynchronizer;
enum class ETimecodeSynchronizationEvent;

PRAGMA_DISABLE_DEPRECATION_WARNINGS

/** Viewer/editor for a TimecodeSynchronizer */
class FTimecodeSynchronizerEditorToolkit : public FAssetEditorToolkit
{
private:
	using Super = FAssetEditorToolkit;

public:
	static TSharedRef<FTimecodeSynchronizerEditorToolkit> CreateEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UTimecodeSynchronizer* InTimecodeSynchronizer);

	/**
	 * Edits the specified table
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	InTimecodeSynchronizer	The TimecodeSynchronizer asset to edit
	 */
	void InitTimecodeSynchronizerEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UTimecodeSynchronizer* InTimecodeSynchronizer);
	~FTimecodeSynchronizerEditorToolkit();

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual void RemoveEditingObject(UObject* Object) override;

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	/** Get the TimecodeSynchronizer Asset being edited */
	class UTimecodeSynchronizer* GetTimecodeSynchronizer() const;

private:
	void HandleAssetPostImport(UFactory* InFactory, UObject* InObject);
	TSharedRef<SDockTab> SpawnPropertiesTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnSourceViewerTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnSynchronizerWidgetTab(const FSpawnTabArgs& Args);

	void ExtendToolBar();

	FLinearColor GetProgressColor() const;
	void HandleSynchronizationEvent(ETimecodeSynchronizationEvent Event);

private:
	/** Details view */
	TSharedPtr<class IDetailsView> DetailsView;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
