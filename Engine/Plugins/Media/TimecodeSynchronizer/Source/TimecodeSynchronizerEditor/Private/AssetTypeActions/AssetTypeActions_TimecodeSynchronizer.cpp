// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTypeActions/AssetTypeActions_TimecodeSynchronizer.h"

#include "AssetEditor/TimecodeSynchronizerEditorToolkit.h"
#include "TimecodeSynchronizer.h"

#define LOCTEXT_NAMESPACE "TimecodeSynchronizerEditor"
PRAGMA_DISABLE_DEPRECATION_WARNINGS
FText FAssetTypeActions_TimecodeSynchronizer::GetName() const
{
	return LOCTEXT("AssetTypeActions_TimecodeSynchronizer", "Timecode Synchronizer");
}

UClass* FAssetTypeActions_TimecodeSynchronizer::GetSupportedClass() const
{
	return UTimecodeSynchronizer::StaticClass();
}

void FAssetTypeActions_TimecodeSynchronizer::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor)
{
	for (UObject* Obj : InObjects)
	{
		if (UTimecodeSynchronizer* Asset = Cast<UTimecodeSynchronizer>(Obj))
		{
			FTimecodeSynchronizerEditorToolkit::CreateEditor(EToolkitMode::Standalone, EditWithinLevelEditor, Asset);
		}
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#undef LOCTEXT_NAMESPACE
