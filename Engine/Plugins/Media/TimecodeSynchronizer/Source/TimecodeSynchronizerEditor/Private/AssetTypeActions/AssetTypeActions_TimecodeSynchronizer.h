// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions_Base.h"

class FAssetTypeActions_TimecodeSynchronizer : public FAssetTypeActions_Base
{
public:
	virtual FColor GetTypeColor() const override { return FColor(140, 62, 35); }
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Media; }
	virtual bool IsImportedAsset() const override { return false; }

	virtual FText GetName() const override;
	virtual UClass* GetSupportedClass() const override;
	virtual void OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>()) override;
};
