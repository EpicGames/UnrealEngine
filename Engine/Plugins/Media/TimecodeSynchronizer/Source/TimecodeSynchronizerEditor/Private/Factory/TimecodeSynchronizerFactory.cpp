// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimecodeSynchronizerFactory.h"

#include "TimecodeSynchronizer.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UTimecodeSynchronizerFactory::UTimecodeSynchronizerFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UTimecodeSynchronizer::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UTimecodeSynchronizerFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	ensure(UTimecodeSynchronizer::StaticClass() == Class);
	ensure(0 != (RF_Public & Flags));
	UTimecodeSynchronizer* TimecodeSynchronizer = NewObject<UTimecodeSynchronizer>(InParent, Name, Flags);
	return TimecodeSynchronizer;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS