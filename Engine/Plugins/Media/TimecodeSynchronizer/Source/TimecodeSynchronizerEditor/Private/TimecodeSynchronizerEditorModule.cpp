// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimecodeSynchronizerEditorModule.h"

#include "AssetTypeActions/AssetTypeActions_TimecodeSynchronizer.h"
#include "TimecodeSynchronizerProjectSettings.h"
#include "UI/TimecodeSynchronizerEditorCommand.h"
#include "UI/TimecodeSynchronizerEditorLevelToolbar.h"
#include "UI/TimecodeSynchronizerEditorStyle.h"

#include "ISettingsModule.h"
#include "ISettingsSection.h"

#define LOCTEXT_NAMESPACE "TimecodeSynchronizerEditor"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

//////////////////////////////////////////////////////////////////////////
// FTimecodeSynchronizerEditorModule
class FTimecodeSynchronizerEditorModule : public ITimecodeSynchronizerEditorModule
{
public:
	virtual void StartupModule() override
	{
		LLM_SCOPE_BYNAME(TEXT("TimecodeSynchronizerEditor"));
		// Disable any UI feature if running in command mode
		if (!IsRunningCommandlet())
		{
			FTimecodeSynchronizerEditorStyle::Register();
			FTimecodeSynchronizerEditorCommand::Register();
			if (GetDefault<UTimecodeSynchronizerProjectSettings>()->bDisplayInToolbar)
			{
				LevelToolbar = new FTimecodeSynchronizerEditorLevelToolbar();
			}

			// Register AssetTypeActions
			AssetTypeAction = MakeShareable(new FAssetTypeActions_TimecodeSynchronizer());
			FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get().RegisterAssetTypeActions(AssetTypeAction.ToSharedRef());

			// Register settings
			ISettingsSectionPtr SettingsSection = FModuleManager::GetModulePtr<ISettingsModule>("Settings")->RegisterSettings("Project", "Plugins", "TimecodeSynchronizer",
				LOCTEXT("TimecodeSynchronizerSettingsName", "Timecode Synchronizer"),
				LOCTEXT("TimecodeSynchronizerSettingsDescription", "Configure the TimecodeSynchronizer plug-in."),
				GetMutableDefault<UTimecodeSynchronizerProjectSettings>()
				);
		}
	}

	virtual void ShutdownModule() override
	{
		if (!IsRunningCommandlet() && UObjectInitialized() && !IsEngineExitRequested())
		{
			// Unregister settings
			FModuleManager::GetModulePtr<ISettingsModule>("Settings")->UnregisterSettings("Project", "Plugins", "TimecodeSynchronizer");

			// Unregister AssetTypeActions
			FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get().UnregisterAssetTypeActions(AssetTypeAction.ToSharedRef());

			FTimecodeSynchronizerEditorCommand::Unregister();
			FTimecodeSynchronizerEditorStyle::Unregister();
		}

		delete LevelToolbar;
	}

	FTimecodeSynchronizerEditorLevelToolbar* LevelToolbar = nullptr;
	TSharedPtr<FAssetTypeActions_TimecodeSynchronizer> AssetTypeAction;
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FTimecodeSynchronizerEditorModule, TimecodeSynchronizerEditor);

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
