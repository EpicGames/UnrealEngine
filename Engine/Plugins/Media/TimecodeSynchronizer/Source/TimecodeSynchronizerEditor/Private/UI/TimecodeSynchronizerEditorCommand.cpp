// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/TimecodeSynchronizerEditorCommand.h"

#include "AssetEditor/TimecodeSynchronizerEditorToolkit.h"
#include "Framework/Commands/UICommandList.h"
#include "TimecodeSynchronizerProjectSettings.h"
#include "TimecodeSynchronizer.h"
#include "UI/TimecodeSynchronizerEditorStyle.h"


#define LOCTEXT_NAMESPACE "TimecodeSynchronizerEditor"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

//////////////////////////////////////////////////////////////////////////
// FTimecodeSynchronizerEditorCommand

FTimecodeSynchronizerEditorCommand::FTimecodeSynchronizerEditorCommand()
	: TCommands<FTimecodeSynchronizerEditorCommand>(
		"TimecodeSynchronizerEditorCommands", // Context name for fast lookup
		NSLOCTEXT("TimecodeSynchronizerEditorCommands", "TimecodeSynchronizer Commands", "TimecodeSynchronizer Commands"), // Localized context name for displaying
		FName(),
		FTimecodeSynchronizerEditorStyle::GetStyleSetName() // Icon Style Set
		)
{
}

void FTimecodeSynchronizerEditorCommand::RegisterCommands()
{
	UI_COMMAND(OpenEditorCommand, "Timecode Synchronizer", "Open TimecodeSynchronizer Editor", EUserInterfaceActionType::Button, FInputChord());

	// Action to open the TimecodeSynchronizerEditor
	CommandActionList = MakeShareable(new FUICommandList);

	CommandActionList->MapAction(
		OpenEditorCommand,
		FExecuteAction::CreateStatic(&FTimecodeSynchronizerEditorCommand::OpenTimecodeSynchronizerEditor),
		FCanExecuteAction::CreateStatic(&FTimecodeSynchronizerEditorCommand::CanOpenTimecodeSynchronizerEditor)
	);
}

//////////////////////////////////////////////////////////////////////////
// OpenConsoleCommand

void FTimecodeSynchronizerEditorCommand::OpenTimecodeSynchronizerEditor()
{
	TSoftObjectPtr<UTimecodeSynchronizer>& TimecodeSynchronizer = UTimecodeSynchronizerProjectSettings::StaticClass()->GetDefaultObject<UTimecodeSynchronizerProjectSettings>()->DefaultTimecodeSynchronizer;
	if (!TimecodeSynchronizer.IsNull())
	{
		if (UTimecodeSynchronizer* Asset = TimecodeSynchronizer.LoadSynchronous())
		{
			FTimecodeSynchronizerEditorToolkit::CreateEditor(EToolkitMode::Standalone, nullptr, Asset);
		}
	}
}

bool FTimecodeSynchronizerEditorCommand::CanOpenTimecodeSynchronizerEditor()
{
	return !UTimecodeSynchronizerProjectSettings::StaticClass()->GetDefaultObject<UTimecodeSynchronizerProjectSettings>()->DefaultTimecodeSynchronizer.IsNull();
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
