// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

//////////////////////////////////////////////////////////////////////////
// FTimecodeSynchronizerEditorCommand

class FTimecodeSynchronizerEditorCommand : public TCommands<FTimecodeSynchronizerEditorCommand>
{
public:
	FTimecodeSynchronizerEditorCommand();

	/** Describe and instantiate the commands in here by using the UI COMMAND macro. */
	virtual void RegisterCommands() override;

private:
	static void OpenTimecodeSynchronizerEditor();
	static bool CanOpenTimecodeSynchronizerEditor();

public:
	TSharedPtr<FUICommandInfo>		OpenEditorCommand;

	TSharedPtr<class FUICommandList>		CommandActionList;
};