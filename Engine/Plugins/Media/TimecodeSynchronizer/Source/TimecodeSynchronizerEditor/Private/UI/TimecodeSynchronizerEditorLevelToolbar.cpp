// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/TimecodeSynchronizerEditorLevelToolbar.h"

#include "IAssetTools.h"
#include "TimecodeSynchronizerFactory.h"
#include "TimecodeSynchronizer.h"
#include "TimecodeSynchronizerProjectSettings.h"
#include "UI/TimecodeSynchronizerEditorCommand.h"
#include "UI/TimecodeSynchronizerEditorStyle.h"

#include "AssetToolsModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LevelEditor.h"
#include "Misc/FeedbackContext.h"
#include "PropertyCustomizationHelpers.h"



#define LOCTEXT_NAMESPACE "TimecodeSynchronizerEditor"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

//////////////////////////////////////////////////////////////////////////
// FTimecodeSynchronizerEditorLevelToolbar

FTimecodeSynchronizerEditorLevelToolbar::FTimecodeSynchronizerEditorLevelToolbar()
{
	CurrentTimecodeSynchronizer = GetDefault<UTimecodeSynchronizerEditorSettings>()->UserTimecodeSynchronizer.LoadSynchronous();
	if (CurrentTimecodeSynchronizer == nullptr)
	{
		CurrentTimecodeSynchronizer = GetDefault<UTimecodeSynchronizerProjectSettings>()->DefaultTimecodeSynchronizer.LoadSynchronous();
	}

	ExtendLevelEditorToolbar();
}

FTimecodeSynchronizerEditorLevelToolbar::~FTimecodeSynchronizerEditorLevelToolbar()
{
	if (UObjectInitialized() && LevelToolbarExtender.IsValid() && !IsEngineExitRequested())
	{
		// Add a TimecodeSynchronizer toolbar section after the settings section of the level editor
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		if (LevelEditorModule.GetToolBarExtensibilityManager().IsValid())
		{
			LevelEditorModule.GetToolBarExtensibilityManager()->RemoveExtender(LevelToolbarExtender);
		}
	}
}

void FTimecodeSynchronizerEditorLevelToolbar::ExtendLevelEditorToolbar()
{
	check(!LevelToolbarExtender.IsValid());

	// Create Toolbar extension
	LevelToolbarExtender = MakeShareable(new FExtender);
	LevelToolbarExtender->AddToolBarExtension(
		"Settings",
		EExtensionHook::After,
		FTimecodeSynchronizerEditorCommand::Get().CommandActionList,
		FToolBarExtensionDelegate::CreateRaw(this, &FTimecodeSynchronizerEditorLevelToolbar::FillToolbar)
		);

	// Add a TimecodeSynchronizer toolbar section after the settings section of the level editor
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditorModule.GetToolBarExtensibilityManager()->AddExtender(LevelToolbarExtender);
}

void FTimecodeSynchronizerEditorLevelToolbar::FillToolbar(FToolBarBuilder& ToolbarBuilder)
{
	ToolbarBuilder.BeginSection("TimecodeSynchronizer");
	{
		auto TooltipLambda = [this]()
		{
			if (!CurrentTimecodeSynchronizer.IsValid())
			{
				return LOCTEXT("EmptyTimecodeSynchronizer_ToolTip", "Select a Timecode Synchronizer to edit it.");
			}
			return FText::Format(LOCTEXT("TimecodeSynchronizer_ToolTip", "Edit '{0}'")
				, FText::FromName(CurrentTimecodeSynchronizer->GetFName()));
		};

		// Add a button to edit the current timecode synchronizer
		ToolbarBuilder.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateRaw(this, &FTimecodeSynchronizerEditorLevelToolbar::OpenCurrentTimecodeSynchronizer),
				FCanExecuteAction::CreateLambda([this] { return CurrentTimecodeSynchronizer.IsValid(); }),
				FIsActionChecked::CreateLambda([this] { return CurrentTimecodeSynchronizer.IsValid(); })
			),
			NAME_None,
			LOCTEXT("TimecodeSynch_Label", "Timecode Synchronizer"),
			MakeAttributeLambda(TooltipLambda),
			FSlateIcon(FTimecodeSynchronizerEditorStyle::GetStyleSetName(), TEXT("Console"))
		);

		// Add a simple drop-down menu (no label, no icon for the drop-down button itself) that list the timecode synchronizer available
		ToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateRaw(this, &FTimecodeSynchronizerEditorLevelToolbar::GenerateMenuContent),
			FText::GetEmpty(),
			LOCTEXT("TimecodeSynchButton_ToolTip", "List of Timecode Synchronizer available to the user for editing or creation."),
			FSlateIcon(),
			true
		);
	}
	ToolbarBuilder.EndSection();
}

TSharedRef<SWidget> FTimecodeSynchronizerEditorLevelToolbar::GenerateMenuContent()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	MenuBuilder.BeginSection("TimecodeSynchronizer", LOCTEXT("NewTimecodeSynchronizerSection", "New"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CreateMenuLabel", "New Empty Timecode Synchronizer"),
			LOCTEXT("CreateMenuTooltip", "Create a new Timecode Synchronizer asset."),
			FSlateIcon(FTimecodeSynchronizerEditorStyle::GetStyleSetName(), TEXT("Console")),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FTimecodeSynchronizerEditorLevelToolbar::CreateNewTimecodeSynchronizer)
			)
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("TimecodeSynchronizer", LOCTEXT("TimecodeSynchronizerSection", "Timecode Synchronizer"));
	{
		const bool bIsTimecodeSynchronizerValid = CurrentTimecodeSynchronizer.IsValid();

		MenuBuilder.AddSubMenu(
			bIsTimecodeSynchronizerValid ? FText::FromName(CurrentTimecodeSynchronizer->GetFName()) : LOCTEXT("SelectMenuLabel", "Select a Timecode Synchronizer"),
			LOCTEXT("SelectMenuTooltip", "Select the current timecode synchronizer for this editor."),
			FNewMenuDelegate::CreateRaw(this, &FTimecodeSynchronizerEditorLevelToolbar::AddObjectSubMenu),
			FUIAction(),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FTimecodeSynchronizerEditorLevelToolbar::AddObjectSubMenu(FMenuBuilder& MenuBuilder)
{
	FAssetData CurrentAssetData = CurrentTimecodeSynchronizer.IsValid() ? FAssetData(CurrentTimecodeSynchronizer.Get()) : FAssetData();

	TArray<const UClass*> ClassFilters;
	ClassFilters.Add(UTimecodeSynchronizer::StaticClass());

	MenuBuilder.AddWidget(
		PropertyCustomizationHelpers::MakeAssetPickerWithMenu(
			CurrentAssetData,
			CurrentTimecodeSynchronizer.IsValid(),
			false,
			ClassFilters,
			TArray<UFactory*>(),
			FOnShouldFilterAsset::CreateLambda([CurrentAssetData](const FAssetData& InAssetData) { return InAssetData == CurrentAssetData; }),
			FOnAssetSelected::CreateRaw(this, &FTimecodeSynchronizerEditorLevelToolbar::NewTimecodeSynchronizerSelected),
			FSimpleDelegate()
		),
		FText::GetEmpty(),
		true,
		false
	);
}

void FTimecodeSynchronizerEditorLevelToolbar::OpenCurrentTimecodeSynchronizer()
{
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(CurrentTimecodeSynchronizer.Get());
}

void FTimecodeSynchronizerEditorLevelToolbar::CreateNewTimecodeSynchronizer()
{
	UTimecodeSynchronizerFactory* FactoryInstance = DuplicateObject<UTimecodeSynchronizerFactory>(GetDefault<UTimecodeSynchronizerFactory>(), GetTransientPackage());
	FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
	UTimecodeSynchronizer* NewAsset = Cast<UTimecodeSynchronizer>(FAssetToolsModule::GetModule().Get().CreateAssetWithDialog(FactoryInstance->GetSupportedClass(), FactoryInstance));
	if (NewAsset != nullptr)
	{
		GetMutableDefault<UTimecodeSynchronizerEditorSettings>()->UserTimecodeSynchronizer = NewAsset;
		GetMutableDefault<UTimecodeSynchronizerEditorSettings>()->SaveConfig();
		CurrentTimecodeSynchronizer = NewAsset;
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewAsset);
	}
}

void FTimecodeSynchronizerEditorLevelToolbar::NewTimecodeSynchronizerSelected(const FAssetData& AssetData)
{
	FSlateApplication::Get().DismissAllMenus();

	GWarn->BeginSlowTask(LOCTEXT("TImecodeSynchronizerLoadPackage", "Loading Timecode Synchronizer"), true, false);
	UTimecodeSynchronizer* Asset = Cast<UTimecodeSynchronizer>(AssetData.GetAsset());
	GWarn->EndSlowTask();

	GetMutableDefault<UTimecodeSynchronizerEditorSettings>()->UserTimecodeSynchronizer = Asset;
	GetMutableDefault<UTimecodeSynchronizerEditorSettings>()->SaveConfig();
	CurrentTimecodeSynchronizer = Asset;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#undef LOCTEXT_NAMESPACE
