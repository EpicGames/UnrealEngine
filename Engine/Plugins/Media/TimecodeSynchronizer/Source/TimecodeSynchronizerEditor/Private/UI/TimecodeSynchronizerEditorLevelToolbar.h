// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakObjectPtr.h"

class UTimecodeSynchronizer;

struct FAssetData;
class FExtender;
class FMenuBuilder;
class FToolBarBuilder;
class SWidget;

//////////////////////////////////////////////////////////////////////////
// FTimecodeSynchronizerEditorLevelToolbar
PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FTimecodeSynchronizerEditorLevelToolbar
{
public:
	FTimecodeSynchronizerEditorLevelToolbar();
	~FTimecodeSynchronizerEditorLevelToolbar();

private:
	void ExtendLevelEditorToolbar();
	void FillToolbar(FToolBarBuilder& ToolbarBuilder);

	void AddObjectSubMenu(FMenuBuilder& MenuBuilder);
	TSharedRef<SWidget> GenerateMenuContent();
	void OpenCurrentTimecodeSynchronizer();
	void CreateNewTimecodeSynchronizer();
	void NewTimecodeSynchronizerSelected(const FAssetData& AssetData);


private:
	TSharedPtr<FExtender> LevelToolbarExtender;
	TWeakObjectPtr<UTimecodeSynchronizer> CurrentTimecodeSynchronizer;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
