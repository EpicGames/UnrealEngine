// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/TimecodeSynchronizerEditorStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

namespace TimecodeSynchronizerStyle
{
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);


	const FName NAME_StyleName(TEXT("TimecodeSynchronizerStyle"));
	const FName NAME_ContextName(TEXT("TimecodeSynchronizer"));

	static TUniquePtr<FSlateStyleSet> StyleInstance;
}

#define IMAGE_BRUSH(RelativePath, ...) FSlateImageBrush(TimecodeSynchronizerStyle::StyleInstance->RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)

void FTimecodeSynchronizerEditorStyle::Register()
{
	TimecodeSynchronizerStyle::StyleInstance = MakeUnique<FSlateStyleSet>(TimecodeSynchronizerStyle::NAME_StyleName);
	TimecodeSynchronizerStyle::StyleInstance->SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Media/TimecodeSynchronizer/Content/Editor/Icons"));

	TimecodeSynchronizerStyle::StyleInstance->Set("ClassThumbnail.TimecodeSynchronizer", new IMAGE_BRUSH("TimecodeSynchronizer_64x", TimecodeSynchronizerStyle::Icon64x64));
	TimecodeSynchronizerStyle::StyleInstance->Set("ClassIcon.TimecodeSynchronizer", new IMAGE_BRUSH("TimecodeSynchronizer_20x", TimecodeSynchronizerStyle::Icon20x20));

	TimecodeSynchronizerStyle::StyleInstance->Set("Console", new IMAGE_BRUSH("Icon_TimecodeSynchronizer_40x", TimecodeSynchronizerStyle::Icon40x40));
	TimecodeSynchronizerStyle::StyleInstance->Set("Console.Small", new IMAGE_BRUSH("Icon_TimecodeSynchronizer_20x", TimecodeSynchronizerStyle::Icon20x20));
	TimecodeSynchronizerStyle::StyleInstance->Set("Synchronized", new IMAGE_BRUSH("Icon_Synchronized_40x", TimecodeSynchronizerStyle::Icon40x40));
	TimecodeSynchronizerStyle::StyleInstance->Set("Synchronized.Small", new IMAGE_BRUSH("Icon_Synchronized_40x", TimecodeSynchronizerStyle::Icon20x20));
	TimecodeSynchronizerStyle::StyleInstance->Set("Stop", new IMAGE_BRUSH("Icon_Stop_40x", TimecodeSynchronizerStyle::Icon40x40));
	TimecodeSynchronizerStyle::StyleInstance->Set("Stop.Small", new IMAGE_BRUSH("Icon_Stop_40x", TimecodeSynchronizerStyle::Icon20x20));
	TimecodeSynchronizerStyle::StyleInstance->Set("SynchronizationWidget.Small", new IMAGE_BRUSH("Icon_SynchronizationWidget_20x", TimecodeSynchronizerStyle::Icon20x20));

	FSlateStyleRegistry::RegisterSlateStyle(*TimecodeSynchronizerStyle::StyleInstance.Get());
}

void FTimecodeSynchronizerEditorStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*TimecodeSynchronizerStyle::StyleInstance.Get());
	TimecodeSynchronizerStyle::StyleInstance.Reset();
}

FName FTimecodeSynchronizerEditorStyle::GetStyleSetName()
{
	return TimecodeSynchronizerStyle::NAME_StyleName;
}

const ISlateStyle& FTimecodeSynchronizerEditorStyle::Get()
{
	check(TimecodeSynchronizerStyle::StyleInstance.IsValid());
	return *TimecodeSynchronizerStyle::StyleInstance.Get();
}

#undef IMAGE_BRUSH
