// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/STimecodeSynchronizerSourceViewer.h"


#include "TimecodeSynchronizer.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/STimecodeSynchronizerSourceViewport.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
/* STimecodeSynchronizerSourceViewer structors
 *****************************************************************************/

STimecodeSynchronizerSourceViewer::STimecodeSynchronizerSourceViewer()
	: TimecodeSynchronizer(nullptr)
	, ViewportVerticalBox(nullptr)
{

}

STimecodeSynchronizerSourceViewer::~STimecodeSynchronizerSourceViewer()
{
	if (TimecodeSynchronizer.IsValid())
	{
		TimecodeSynchronizer->OnSynchronizationEvent().RemoveAll(this);
	}
}

/* STimecodeSynchronizerSourceViewer implementation
 *****************************************************************************/

void STimecodeSynchronizerSourceViewer::Construct(const FArguments& InArgs, UTimecodeSynchronizer& InTimecodeSynchronization)
{
	TimecodeSynchronizer.Reset(&InTimecodeSynchronization);
	TimecodeSynchronizer->OnSynchronizationEvent().AddSP(this, &STimecodeSynchronizerSourceViewer::HandleSynchronizationEvent);

	//Create the Box that will hold a Widget for each source.
	ViewportVerticalBox = SNew(SVerticalBox);

	// The viewer will be populated later on event.
	ChildSlot
	[
		SNew(SOverlay)

		+ SOverlay::Slot()
		[
			ViewportVerticalBox.ToSharedRef()
		]
	];

	PopulateActiveSources();
}

TSharedRef<SWidget> STimecodeSynchronizerSourceViewer::GetVisualWidget(const FTimecodeSynchronizerActiveTimecodedInputSource& InSource)
{
	if (const UTimeSynchronizationSource* SyncSource = InSource.GetInputSource())
	{
		return SyncSource->GetVisualWidget();
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}

void STimecodeSynchronizerSourceViewer::PopulateActiveSources()
{
	ViewportVerticalBox->ClearChildren();
	
	if (TimecodeSynchronizer.IsValid())
	{
		struct FLocal
		{
			static void BuildViewportsForSources(UTimecodeSynchronizer* Synchronizer, TSharedPtr<SVerticalBox> Owner, const bool bSynchronizedSources)
			{
				const TArray<FTimecodeSynchronizerActiveTimecodedInputSource>& TimecodedSources = bSynchronizedSources ? Synchronizer->GetSynchronizedSources() : Synchronizer->GetNonSynchronizedSources();
				for (int32 Index = 0; Index < TimecodedSources.Num(); ++Index)
				{
					const FTimecodeSynchronizerActiveTimecodedInputSource& Source = TimecodedSources[Index];

					//Add a Viewport Widget for each active Source
					Owner->AddSlot()
						.Padding(1.0f, 1.0f, 1.0f, 1.0f)
						[
							SNew(SBorder)
							.BorderImage(FCoreStyle::Get().GetBrush("GreenBrush"))
							.Padding(0.0f)
							[
								//Source area
								SNew(STimecodeSynchronizerSourceViewport, Synchronizer, Index, bSynchronizedSources, GetVisualWidget(TimecodedSources[Index]))
							]
						];
				}
			}
		};

		FLocal::BuildViewportsForSources(TimecodeSynchronizer.Get(), ViewportVerticalBox, true);
		FLocal::BuildViewportsForSources(TimecodeSynchronizer.Get(), ViewportVerticalBox, false);
	}
}

void STimecodeSynchronizerSourceViewer::HandleSynchronizationEvent(ETimecodeSynchronizationEvent Event)
{
	if (Event == ETimecodeSynchronizationEvent::SynchronizationSucceeded || Event == ETimecodeSynchronizationEvent::SynchronizationStopped)
	{
		PopulateActiveSources();
	}
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
