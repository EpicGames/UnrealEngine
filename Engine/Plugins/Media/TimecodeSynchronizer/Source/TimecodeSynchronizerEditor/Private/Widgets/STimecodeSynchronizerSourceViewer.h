// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "UObject/StrongObjectPtr.h"

class UTimecodeSynchronizer;
enum class ETimecodeSynchronizationEvent;
struct FTimecodeSynchronizerActiveTimecodedInputSource;

class FMenuBuilder;
class SEditableTextBox;
class STimecodeSynchronizerSourceViewport;
class SVerticalBox;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
/**
 * Implements the contents of the viewer tab in the TimecodeSynchronizer editor.
 */
class STimecodeSynchronizerSourceViewer
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(STimecodeSynchronizerSourceViewer) { }
	SLATE_END_ARGS()

public:

	/** Default constructor. */
	STimecodeSynchronizerSourceViewer();
	
	/** Default destructor. */
	virtual ~STimecodeSynchronizerSourceViewer();
	
public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InTimecodeSynchronizer The active Asset.
	 */
	void Construct(const FArguments& InArgs, UTimecodeSynchronizer& InTimecodeSynchronizer);

private:

	void PopulateActiveSources();
	void HandleSynchronizationEvent(ETimecodeSynchronizationEvent Event);
	static TSharedRef<SWidget> GetVisualWidget(const FTimecodeSynchronizerActiveTimecodedInputSource& InSource);

private:

	/** Active TimecodeSynchronizer */
	TStrongObjectPtr<UTimecodeSynchronizer> TimecodeSynchronizer;

	/** VerticalBox holding active sources viewport */
	TSharedPtr<SVerticalBox> ViewportVerticalBox;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
