// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/STimecodeSynchronizerSourceViewport.h"
#include "TimecodeSynchronizer.h"

#include "Misc/App.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"



#define LOCTEXT_NAMESPACE "STimecodeSynchronizercSourceViewport"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

/* STimecodeSynchronizerSourceViewport structors
*****************************************************************************/

STimecodeSynchronizerSourceViewport::STimecodeSynchronizerSourceViewport()
	: SourceTextBox(nullptr)
	, TimecodeSynchronization(nullptr)
	, AttachedSourceIndex(INDEX_NONE)
	, bIsSynchronizedSource(false)
{ }

/* STimecodeSynchronizerSourceViewport interface
*****************************************************************************/

void STimecodeSynchronizerSourceViewport::Construct(const FArguments& InArgs, UTimecodeSynchronizer* InTimecodeSynchronizer, int32 InAttachedSourceIndex, bool bInIsSynchronizedSource, TSharedRef<SWidget> InVisualWidget)
{
	TimecodeSynchronization.Reset(InTimecodeSynchronizer);
	AttachedSourceIndex = InAttachedSourceIndex;
	bIsSynchronizedSource = bInIsSynchronizedSource;

	FSlateFontInfo Font18 = FCoreStyle::Get().GetFontStyle(TEXT("NormalFont"));
	Font18.Size = 18;

	ChildSlot
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				//Source display name
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(1.0f, 1.0f, 1.0f, 1.0f)
				[
					// Display name box
					SAssignNew(SourceTextBox, SEditableTextBox)
					.ClearKeyboardFocusOnCommit(true)
					.HintText(LOCTEXT("SourceTextBoxHint", "Source Name"))
					.IsReadOnly(true)
					.Text_Lambda([&]() -> FText
					{
						const FTimecodeSynchronizerActiveTimecodedInputSource* AttachedSource = GetAttachedSource();
						if (AttachedSource && AttachedSource->IsReady())
						{
							return FText::FromString(GetAttachedSource()->GetDisplayName());
						}
						return FText();
					})
				]
			]
		
			+ SVerticalBox::Slot()
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					InVisualWidget
				]

				+ SOverlay::Slot()
				.Padding(FMargin(12.0f, 8.0f))
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.VAlign(VAlign_Top)
					[

						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Right)
						[
							//Source Timecode Interval
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							.VAlign(VAlign_Top)
							[
								// Min Timecode
								SNew(STextBlock)
								.ColorAndOpacity(FSlateColor::UseSubduedForeground())
								.Font(Font18)
								.ShadowOffset(FVector2D(1.f, 1.f))
								.Text(this, &STimecodeSynchronizerSourceViewport::HandleIntervalMinTimecodeText)
								.Justification(ETextJustify::Right)
								.ToolTipText(LOCTEXT("OverlayMinTimecodeDataTooltip", "Buffered minimum Timecode of this source"))
							]
							
							+ SVerticalBox::Slot()
							.VAlign(VAlign_Bottom)
							[
								// Max Timecode
								SNew(STextBlock)
								.ColorAndOpacity(FSlateColor::UseSubduedForeground())
								.Font(Font18)
								.ShadowOffset(FVector2D(1.f, 1.f))
								.Text(this, &STimecodeSynchronizerSourceViewport::HandleIntervalMaxTimecodeText)
								.Justification(ETextJustify::Right)
								.ToolTipText(LOCTEXT("OverlayMaxTimecodeDataTooltip", "Buffered maximum Timecode of this source"))
							]
						]
					]
					
					+ SVerticalBox::Slot()
					.VAlign(VAlign_Bottom)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Bottom)
						[
							// Display if source is the owner
							SNew(STextBlock)
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
							.Font(Font18)
							.ShadowOffset(FVector2D(1.f, 1.f))
							.Text(this, &STimecodeSynchronizerSourceViewport::HandleIsSourceMainText)
							.ToolTipText(LOCTEXT("OverlayOwnerSourceTooltip", "Is this source used as the owner"))
						]
						
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Bottom)
						[
							// Current Timecode
							SNew(STextBlock)
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
							.Font(Font18)
							.ShadowOffset(FVector2D(1.f, 1.f))
							.Text(this, &STimecodeSynchronizerSourceViewport::HandleCurrentTimecodeText)
							.Justification(ETextJustify::Right)
							.ToolTipText(LOCTEXT("OverlayCurrentTimecodeDataTooltip", "Current Timecode of system"))
						]
					]
				]
			]
		]
	];
}


/* STimecodeSynchronizerSourceViewport callbacks
*****************************************************************************/
FText STimecodeSynchronizerSourceViewport::HandleIntervalMinTimecodeText() const
{
	FTimecode Timecode;
	const FTimecodeSynchronizerActiveTimecodedInputSource* AttachedSource = GetAttachedSource();
	if (AttachedSource && AttachedSource->IsReady())
	{
		const FFrameNumber OldestFrame = AttachedSource->GetInputSourceState().OldestAvailableSample.RoundToFrame();
		Timecode = FTimecode::FromFrameNumber(OldestFrame, AttachedSource->GetFrameRate());
	}

	return FText::FromString(FString("Minimum: ") + Timecode.ToString());
}

FText STimecodeSynchronizerSourceViewport::HandleIntervalMaxTimecodeText() const
{
	FTimecode Timecode;
	const FTimecodeSynchronizerActiveTimecodedInputSource* AttachedSource = GetAttachedSource();
	if (AttachedSource && AttachedSource->IsReady())
	{
		const FFrameNumber NewestFrame = AttachedSource->GetInputSourceState().NewestAvailableSample.RoundToFrame();
		Timecode = FTimecode::FromFrameNumber(NewestFrame, AttachedSource->GetFrameRate());
	}

	return FText::FromString(FString("Maximum: ") + Timecode.ToString());
}

FText STimecodeSynchronizerSourceViewport::HandleCurrentTimecodeText() const
{
	return FText::FromString(FString("Current: ") + FApp::GetTimecode().ToString());
}

FText STimecodeSynchronizerSourceViewport::HandleIsSourceMainText() const
{
	FText Role;
	if (TimecodeSynchronization && AttachedSourceIndex != INDEX_NONE && bIsSynchronizedSource && TimecodeSynchronization->GetActiveMainSynchronizationTimecodedSourceIndex() == AttachedSourceIndex)
	{
		Role = LOCTEXT("MainLabel", "Main");
	}

	return Role;
}

const FTimecodeSynchronizerActiveTimecodedInputSource* STimecodeSynchronizerSourceViewport::GetAttachedSource() const
{
	const FTimecodeSynchronizerActiveTimecodedInputSource* Result = nullptr;

	if (TimecodeSynchronization && AttachedSourceIndex != INDEX_NONE)
	{
		const TArray<FTimecodeSynchronizerActiveTimecodedInputSource>& Sources = bIsSynchronizedSource ? TimecodeSynchronization->GetSynchronizedSources() : TimecodeSynchronization->GetNonSynchronizedSources();
		if (Sources.IsValidIndex(AttachedSourceIndex))
		{
			Result = &Sources[AttachedSourceIndex];
		}
	}

	return Result;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
