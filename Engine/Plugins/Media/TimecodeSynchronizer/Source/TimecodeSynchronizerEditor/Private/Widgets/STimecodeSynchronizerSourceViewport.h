// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "UObject/StrongObjectPtr.h"

class SEditableTextBox;
struct FSlateBrush;
struct FTimecodeSynchronizerActiveTimecodedInputSource;
class UTimecodeSynchronizer;

PRAGMA_DISABLE_DEPRECATION_WARNINGS

class STimecodeSynchronizerSourceViewport
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(STimecodeSynchronizerSourceViewport) { }
	SLATE_END_ARGS()

public:

	/** Default constructor. */
	STimecodeSynchronizerSourceViewport();

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InTimecodeSynchronization The Asset currently active.
	 * @param InAttachedSource The Source to display.
	 * @param InTexture The source texture to render if any
	 */
	void Construct(const FArguments& InArgs, UTimecodeSynchronizer* InTimecodeSynchronization, int32 InAttachedSourceIndex, bool InTimecodedSource, TSharedRef<SWidget> InVisualWidget);

private:

	/** Callback for getting the overlay text of Source minimum Timecode buffered. */
	FText HandleIntervalMinTimecodeText() const;

	/** Callback for getting the overlay text of Source maximum Timecode buffered. */
	FText HandleIntervalMaxTimecodeText() const;
	
	/** Callback for getting the text of the Current Synchronized Timecode. */
	FText HandleCurrentTimecodeText() const;
	
	/** Callback setuping the text if the source is the main source. */
	FText HandleIsSourceMainText() const;

	/** Get the attached Input source in TimecodeSynchronization.GetActiveSources() at the AttachedSourceIndex index  */
	const FTimecodeSynchronizerActiveTimecodedInputSource* GetAttachedSource() const;

private:

	/** Media Source name text box. */
	TSharedPtr<SEditableTextBox> SourceTextBox;

	/** Current TimecodeSynchronization being used. */
	TStrongObjectPtr<UTimecodeSynchronizer> TimecodeSynchronization;

	/** Attached Input source index in either TimecodeSynchronization.GetSynchronizedSources() or GetNonSynchronizedSources(). */
	int32 AttachedSourceIndex;

	/** Whether or not this source is used for synchronization. */
	bool bIsSynchronizedSource;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
