// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimecodeSynchronizerWidget.h"


#include "Fonts/FontMeasure.h"

#include "Rendering/SlateRenderer.h"
#include "TimecodeSynchronizer.h"


#include "Widgets/Input/SSpinBox.h"

#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSplitter.h"

namespace TimecodeSynchronizerBarWidget
{
	const int32 Width = 200;
	const int32 RowHeight = 20;
	const int32 TextSize = 10;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

STimecodeSynchronizerBarWidget::STimecodeSynchronizerBarWidget()
	: SWidget()
	, CurrentFrameWidth(10)
	, CurrentFrameValue(0)
	, CurrentFrame(0)
	, CurrentOwnerIndex(0)
	, MinOldestFrameTime(0)
	, MaxOldestFrameTime(0)
	, MinNewestFrameTime(0)
	, MaxNewestFrameTime(0)
	, DarkBrush(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
	, BrightBrush(&FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox").BackgroundImageNormal)
	, FontMeasureService(FSlateApplication::Get().GetRenderer()->GetFontMeasureService())
	, FontInfo(FCoreStyle::GetDefaultFontStyle("Regular", TimecodeSynchronizerBarWidget::TextSize))
	, ColorSynchronizedSamples(0.0f, 0.5f, 0.0f, 1.0f)
	, ColorFutureSamples(0.5f, 0.5f, 0.0f, 1.0f)
	, ColorPastSamples(0.5f, 0.25f, 0.0f, 1.0f)
{
	bCanHaveChildren = false;
}

FVector2D STimecodeSynchronizerBarWidget::ComputeDesiredSize(float) const
{
	return FVector2D(TimecodeSynchronizerBarWidget::Width, (TimecodeSynchronizer->GetSynchronizedSources().Num() + 1) * TimecodeSynchronizerBarWidget::RowHeight);
}

FChildren* STimecodeSynchronizerBarWidget::GetChildren()
{
	return nullptr;
}

void STimecodeSynchronizerBarWidget::OnArrangeChildren(const FGeometry&, FArrangedChildren&) const
{
}

void STimecodeSynchronizerBarWidget::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if ((GFrameNumber % UpdateRate.Get()) == 0)
	{
		CurrentFrameWidth = FrameWidth.Get();

		int32 SourceCount = TimecodeSynchronizer->GetSynchronizedSources().Num();
		if (SourceCount)
		{
			const FTimecodeSourceState& FirstSynchronizerRelativeState = TimecodeSynchronizer->GetSynchronizedSources()[0].GetSynchronizerRelativeState();
			MinOldestFrameTime = FirstSynchronizerRelativeState.OldestAvailableSample.GetFrame().Value;
			MaxOldestFrameTime = FirstSynchronizerRelativeState.OldestAvailableSample.GetFrame().Value;
			MinNewestFrameTime = FirstSynchronizerRelativeState.NewestAvailableSample.GetFrame().Value;
			MaxNewestFrameTime = FirstSynchronizerRelativeState.NewestAvailableSample.GetFrame().Value;

			for (int32 i = 1; i < SourceCount; i++)
			{
				const FTimecodeSourceState& SynchronizerRelativeState = TimecodeSynchronizer->GetSynchronizedSources()[i].GetSynchronizerRelativeState();
				MinOldestFrameTime = FMath::Min(MinOldestFrameTime, SynchronizerRelativeState.OldestAvailableSample.GetFrame().Value);
				MaxOldestFrameTime = FMath::Max(MaxOldestFrameTime, SynchronizerRelativeState.OldestAvailableSample.GetFrame().Value);
				MinNewestFrameTime = FMath::Min(MinNewestFrameTime, SynchronizerRelativeState.NewestAvailableSample.GetFrame().Value);
				MaxNewestFrameTime = FMath::Max(MaxNewestFrameTime, SynchronizerRelativeState.NewestAvailableSample.GetFrame().Value);
			}

			MinOldestFrameTime *= CurrentFrameWidth;
			MaxOldestFrameTime *= CurrentFrameWidth;
			MinNewestFrameTime *= CurrentFrameWidth;
			MaxNewestFrameTime *= CurrentFrameWidth;
		}

		DisplayData.Empty();

		for (int32 i = 0; i < SourceCount; i++)
		{
			FrameTimeDisplayData DisplayDataItem;
			const FTimecodeSynchronizerActiveTimecodedInputSource& SynchronizedSource = TimecodeSynchronizer->GetSynchronizedSources()[i];
			const FTimecodeSourceState& SynchronizerRelativeState = SynchronizedSource.GetSynchronizerRelativeState();
			DisplayDataItem.OldestFrameTime = SynchronizerRelativeState.OldestAvailableSample.GetFrame().Value * CurrentFrameWidth;
			DisplayDataItem.NewestFrameTime = SynchronizerRelativeState.NewestAvailableSample.GetFrame().Value * CurrentFrameWidth;
			DisplayDataItem.Name = SynchronizedSource.GetDisplayName();
			DisplayData.Add(DisplayDataItem);
		}

		CurrentFrameValue = TimecodeSynchronizer->GetCurrentSystemFrameTime().GetFrame().Value;
		CurrentFrameRate = TimecodeSynchronizer->GetFrameRate();
		CurrentFrame = TimecodeSynchronizer->GetCurrentSystemFrameTime().GetFrame().Value * CurrentFrameWidth;
		CurrentOwnerIndex = TimecodeSynchronizer->GetActiveMainSynchronizationTimecodedSourceIndex();
	}
}

int32 STimecodeSynchronizerBarWidget::OnPaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InCullingRect, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const
{
	if (TimecodeSynchronizer.IsValid())
	{
		const int32 Delta = InAllottedGeometry.GetLocalSize().X;

		if (Delta <= 0)
		{
			return InLayerId;
		}

		InLayerId++;

		const int32 SourceCount = DisplayData.Num();

		if (SourceCount)
		{
			const int32 HeaderOffsetY = InAllottedGeometry.GetLocalSize().Y / (SourceCount + 1);
			const int32 SizeY = InAllottedGeometry.GetLocalSize().Y - HeaderOffsetY;

			const int32 TextMinX = 0;
			const int32 TextMaxX = InAllottedGeometry.GetLocalSize().X;

			DrawText(
				InAllottedGeometry,
				OutDrawElements,
				InLayerId,
				CenterJustification,
				FTimecode::FromFrameNumber(CurrentFrameValue, CurrentFrameRate, false).ToString() + FString(TEXT(" (0)")),
				FBox2D(FVector2D(TextMinX, 0), FVector2D(TextMaxX, HeaderOffsetY)),
				InWidgetStyle);

			const int32 FrameValueOffset = int32((InAllottedGeometry.GetLocalSize().X / 2) / CurrentFrameWidth);

			DrawText(
				InAllottedGeometry,
				OutDrawElements,
				InLayerId,
				LeftJustification,
				FTimecode::FromFrameNumber(CurrentFrameValue - FrameValueOffset, CurrentFrameRate, false).ToString() + FString::Printf(TEXT(" (%d)"), -FrameValueOffset),
				FBox2D(FVector2D(TextMinX, 0), FVector2D(TextMaxX, HeaderOffsetY)),
				InWidgetStyle);

			DrawText(
				InAllottedGeometry,
				OutDrawElements,
				InLayerId,
				RightJustification,
				FTimecode::FromFrameNumber(CurrentFrameValue + FrameValueOffset, CurrentFrameRate, false).ToString() + FString::Printf(TEXT(" (%d)"), FrameValueOffset),
				FBox2D(FVector2D(TextMinX, 0), FVector2D(TextMaxX, HeaderOffsetY)),
				InWidgetStyle);

			for (int32 i = 0; i < SourceCount + 1; i++)
			{
				DrawBoxWithBorder(
					InAllottedGeometry,
					OutDrawElements, 
					InLayerId,
					0, 
					InAllottedGeometry.GetLocalSize().X,
					i * SizeY / SourceCount, 
					i * SizeY / SourceCount + HeaderOffsetY, 
					InWidgetStyle, 
					DarkBrush,
					InWidgetStyle.GetColorAndOpacityTint(),
					DarkBrush,
					InWidgetStyle.GetForegroundColor());
			}

			const float ValidMinX = FMath::Clamp(InAllottedGeometry.GetLocalSize().X / 2 + InAllottedGeometry.GetLocalSize().X * (MaxOldestFrameTime - CurrentFrame) / Delta, 0.0f, InAllottedGeometry.GetLocalSize().X);
			const float ValidMaxX = FMath::Clamp(InAllottedGeometry.GetLocalSize().X / 2 + InAllottedGeometry.GetLocalSize().X * (MinNewestFrameTime - CurrentFrame) / Delta, 0.0f, InAllottedGeometry.GetLocalSize().X);

			for (int32 i = 0; i < SourceCount; i++)
			{
				float MinX = FMath::Clamp(InAllottedGeometry.GetLocalSize().X / 2 + InAllottedGeometry.GetLocalSize().X * (DisplayData[i].OldestFrameTime - CurrentFrame) / Delta, 0.0f, InAllottedGeometry.GetLocalSize().X);
				float MaxX = FMath::Clamp(InAllottedGeometry.GetLocalSize().X / 2 + InAllottedGeometry.GetLocalSize().X * (DisplayData[i].NewestFrameTime - CurrentFrame) / Delta, 0.0f, InAllottedGeometry.GetLocalSize().X);

				float MinY = i * SizeY / SourceCount + HeaderOffsetY;
				float MaxY = (i + 1) * SizeY / SourceCount + HeaderOffsetY;

				if (MinX < ValidMinX)
				{
					if (MaxX < ValidMinX)
					{
						DrawBoxWithBorder(InAllottedGeometry, OutDrawElements, InLayerId, MinX, MaxX, MinY, MaxY, InWidgetStyle, DarkBrush, InWidgetStyle.GetColorAndOpacityTint(), BrightBrush, ColorPastSamples);
					}
					else if (MaxX < ValidMaxX)
					{
						DrawBoxWithBorder(InAllottedGeometry, OutDrawElements, InLayerId, MinX, ValidMinX, MinY, MaxY, InWidgetStyle, DarkBrush, InWidgetStyle.GetColorAndOpacityTint(), BrightBrush, ColorPastSamples);
						DrawBoxWithBorder(InAllottedGeometry, OutDrawElements, InLayerId, ValidMinX, MaxX, MinY, MaxY, InWidgetStyle, DarkBrush, InWidgetStyle.GetColorAndOpacityTint(), BrightBrush, ColorSynchronizedSamples);
					}
					else
					{
						DrawBoxWithBorder(InAllottedGeometry, OutDrawElements, InLayerId, MinX, ValidMinX, MinY, MaxY, InWidgetStyle, DarkBrush, InWidgetStyle.GetColorAndOpacityTint(), BrightBrush, ColorPastSamples);
						DrawBoxWithBorder(InAllottedGeometry, OutDrawElements, InLayerId, ValidMinX, ValidMaxX, MinY, MaxY, InWidgetStyle, DarkBrush, InWidgetStyle.GetColorAndOpacityTint(), BrightBrush, ColorSynchronizedSamples);
						DrawBoxWithBorder(InAllottedGeometry, OutDrawElements, InLayerId, ValidMaxX, MaxX, MinY, MaxY, InWidgetStyle, DarkBrush, InWidgetStyle.GetColorAndOpacityTint(), BrightBrush, ColorFutureSamples);
					}
				}
				else if (MinX < ValidMaxX)
				{
					if (MaxX < ValidMaxX)
					{
						DrawBoxWithBorder(InAllottedGeometry, OutDrawElements, InLayerId, MinX, MaxX, MinY, MaxY, InWidgetStyle, DarkBrush, InWidgetStyle.GetColorAndOpacityTint(), BrightBrush, ColorSynchronizedSamples);
					}
					else
					{
						DrawBoxWithBorder(InAllottedGeometry, OutDrawElements, InLayerId, MinX, ValidMaxX, MinY, MaxY, InWidgetStyle, DarkBrush, InWidgetStyle.GetColorAndOpacityTint(), BrightBrush, ColorSynchronizedSamples);
						DrawBoxWithBorder(InAllottedGeometry, OutDrawElements, InLayerId, ValidMaxX, MaxX, MinY, MaxY, InWidgetStyle, DarkBrush, InWidgetStyle.GetColorAndOpacityTint(), BrightBrush, ColorFutureSamples);
					}
				}
				else
				{
					DrawBoxWithBorder(InAllottedGeometry, OutDrawElements, InLayerId, MinX, MaxX, MinY, MaxY, InWidgetStyle, DarkBrush, InWidgetStyle.GetColorAndOpacityTint(), BrightBrush, ColorFutureSamples);
				}

				MinX = FMath::Clamp(InAllottedGeometry.GetLocalSize().X / 2 + InAllottedGeometry.GetLocalSize().X * 0 / Delta, 0.0f, InAllottedGeometry.GetLocalSize().X);
				MaxX = FMath::Clamp(InAllottedGeometry.GetLocalSize().X / 2 + InAllottedGeometry.GetLocalSize().X * CurrentFrameWidth / Delta, 0.0f, InAllottedGeometry.GetLocalSize().X);

				if ((MaxOldestFrameTime <= CurrentFrame) && (CurrentFrame <= MinNewestFrameTime))
				{
					DrawBoxWithBorder(InAllottedGeometry, OutDrawElements, InLayerId, MinX, MaxX, MinY, MaxY, InWidgetStyle, DarkBrush, InWidgetStyle.GetColorAndOpacityTint(), BrightBrush, FLinearColor::Green);
				}
				else
				{
					DrawBoxWithBorder(InAllottedGeometry, OutDrawElements, InLayerId, MinX, MaxX, MinY, MaxY, InWidgetStyle, DarkBrush, InWidgetStyle.GetColorAndOpacityTint(), BrightBrush, FLinearColor::Red);
				}

				DrawText(
					InAllottedGeometry,
					OutDrawElements,
					InLayerId,
					LeftJustification,
					FString::Printf(TEXT("[%d] - %s %s (%d)"), i, *DisplayData[i].Name, (i == CurrentOwnerIndex) ? TEXT(" - Owner") : TEXT(""), (DisplayData[i].NewestFrameTime - DisplayData[i].OldestFrameTime) / CurrentFrameWidth),
					FBox2D(FVector2D(TextMinX, MinY), FVector2D(TextMaxX, MaxY)),
					InWidgetStyle);
			}
		}
	}

	return InLayerId;
}

void STimecodeSynchronizerBarWidget::DrawBoxWithBorder(const FGeometry& InAllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 InLayerId, float InMinX, float InMaxX, float InMinY, float InMaxY, const FWidgetStyle& InWidgetStyle, const FSlateBrush* InBorderBrush, const FLinearColor& InBorderTint, const FSlateBrush* InInsideBrush, const FLinearColor& InInsideTint) const
{
	if (((InMaxX - InMinX) <= 0) || ((InMaxY - InMinY) <= 0))
	{
		return;
	}

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		InLayerId,
		InAllottedGeometry.ToPaintGeometry(FVector2D(InMaxX - InMinX, InMaxY - InMinY), FSlateLayoutTransform(FVector2D(InMinX, InMinY))),
		InBorderBrush,
		ESlateDrawEffect::None,
		InInsideBrush->GetTint(InWidgetStyle) * InBorderTint);

	if (((InMaxX - InMinX) <= 2) || ((InMaxY - InMinY) <= 2))
	{
		return;
	}

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		InLayerId,
		InAllottedGeometry.ToPaintGeometry(FVector2D(InMaxX - InMinX-2, InMaxY - InMinY-2), FSlateLayoutTransform(FVector2D(InMinX+1, InMinY+1))),
		InInsideBrush,
		ESlateDrawEffect::None,
		InInsideBrush->GetTint(InWidgetStyle) * InInsideTint);
}

void STimecodeSynchronizerBarWidget::DrawText(const FGeometry& InAllottedGeometry, FSlateWindowElementList& OutDrawElements, uint32 InLayerId, Justification InJustification, const FString& InText, const FBox2D& InRegion, const FWidgetStyle& InWidgetStyle) const
{
	const float BorderOffsetX = 3;
	FVector2D Size = FontMeasureService->Measure(InText, FontInfo);
	float VOffset = InRegion.Min.Y + (InRegion.GetSize().Y - Size.Y) / 2;
	float HOffset = BorderOffsetX;

	if (InJustification == CenterJustification)
	{
		HOffset = (InRegion.GetSize().X - Size.X) / 2;
	}
	else if (InJustification == RightJustification)
	{
		HOffset = InRegion.GetSize().X - Size.X - BorderOffsetX;
	}

	FSlateDrawElement::MakeText(
		OutDrawElements,
		InLayerId,
		InAllottedGeometry.ToPaintGeometry(InRegion.GetSize() - FVector2D(HOffset, VOffset), FSlateLayoutTransform(FVector2D(HOffset, VOffset))),
		InText,
		FontInfo,
		ESlateDrawEffect::None,
		InWidgetStyle.GetColorAndOpacityTint());

}

void STimecodeSynchronizerBarWidget::Construct(const FArguments& InArgs, UTimecodeSynchronizer& InTimecodeSynchronizer)
{
	UpdateRate = InArgs._UpdateRate;
	FrameWidth = InArgs._FrameWidth;

	TimecodeSynchronizer.Reset(&InTimecodeSynchronizer);
	SetCanTick(true);
}

void STimecodeSynchronizerBarWidget::SetUpdateRate(int32 InUpdateFrequency)
{
	UpdateRate.Set(InUpdateFrequency);
}

void STimecodeSynchronizerBarWidget::SetFrameWidth(int32 InFrameWidth)
{
	FrameWidth.Set(InFrameWidth);
}

void STimecodeSynchronizerWidget::Construct(const FArguments& InArgs, UTimecodeSynchronizer& InTimecodeSynchronizer)
{
	BarWidget = SNew(STimecodeSynchronizerBarWidget, InTimecodeSynchronizer);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.Padding(FMargin(3))
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SSplitter)
				.Orientation(Orient_Horizontal)
				+ SSplitter::Slot()
				.Value(0.3f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FText::FromString(TEXT("Update Rate (Frame)")))
					]
				]
				+ SSplitter::Slot()
				.Value(0.3f)
				[
					SNew(SSpinBox<int32>)
					.MinValue(1)
					.MaxValue(60)
					.Value(1)
					.Delta(1)
					.OnValueChanged(this, &STimecodeSynchronizerWidget::SetUpdateRate)
				]
				+ SSplitter::Slot()
				.Value(0.3f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FText::FromString(TEXT("Frame Display Width (Pixels)")))
					]
				]
				+ SSplitter::Slot()
				.Value(0.3f)
				[
					SNew(SSpinBox<int32>)
					.MinValue(10)
					.MaxValue(100)
					.Value(10)
					.Delta(10)
					.OnValueChanged(this, &STimecodeSynchronizerWidget::SetFrameWidth)
				]
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			BarWidget.ToSharedRef()
		]
	];
}

void STimecodeSynchronizerWidget::SetUpdateRate(int32 InUpdateRate)
{
	BarWidget->SetUpdateRate(InUpdateRate);
}

void STimecodeSynchronizerWidget::SetFrameWidth(int32 InFrameWidth)
{
	BarWidget->SetFrameWidth(InFrameWidth);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
