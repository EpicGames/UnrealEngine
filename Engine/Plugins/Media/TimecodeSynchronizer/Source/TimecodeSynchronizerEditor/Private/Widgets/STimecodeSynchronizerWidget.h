// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"

#include "Fonts/SlateFontInfo.h"
#include "UObject/StrongObjectPtr.h"

#include "Widgets/SCompoundWidget.h"

class UTimecodeSynchronizer;
class FSlateFontMeasure;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
class STimecodeSynchronizerBarWidget : public SWidget
{
public:
	STimecodeSynchronizerBarWidget();

	SLATE_BEGIN_ARGS(STimecodeSynchronizerBarWidget)
		: _UpdateRate(1)
		, _FrameWidth(10)
	{}
	SLATE_ATTRIBUTE(int32, UpdateRate);
	SLATE_ATTRIBUTE(int32, FrameWidth);
	SLATE_END_ARGS()

public:
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual FChildren* GetChildren() override;
	virtual void OnArrangeChildren(const FGeometry&, FArrangedChildren&) const override;
	virtual void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual int32 OnPaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InCullingRect, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const override;
	void Construct(const FArguments& InArgs, UTimecodeSynchronizer& InTimecodeSynchronizer);

public:
	void SetUpdateRate(int32 InUpdateFrequency);
	void SetFrameWidth(int32 InFrameWidth);

private:
	enum Justification
	{
		LeftJustification,
		CenterJustification,
		RightJustification,
	};

	void DrawBoxWithBorder(const FGeometry& InAllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 InLayerId, float InMinX, float InMaxX, float InMinY, float InMaxY, const FWidgetStyle& InWidgetStyle, const FSlateBrush* InBorderBrush, const FLinearColor& InBorderTint, const FSlateBrush* InInsideBrush, const FLinearColor& InInsideTint) const;
	void DrawText(const FGeometry& InAllottedGeometry, FSlateWindowElementList& OutDrawElements, uint32 InLayerId, Justification InJustification, const FString& InText, const FBox2D& InRegion, const FWidgetStyle& InWidgetStyle) const;

private:
	TAttribute<int32> UpdateRate;
	TAttribute<int32> FrameWidth;

	int32 CurrentFrameWidth;
	FFrameRate CurrentFrameRate;
	int32 CurrentFrameValue;
	int32 CurrentFrame;
	int32 CurrentOwnerIndex;

	struct FrameTimeDisplayData
	{
		int32 OldestFrameTime;
		int32 NewestFrameTime;
		FString Name;
	};

	TArray<FrameTimeDisplayData> DisplayData;

	int32 MinOldestFrameTime;
	int32 MaxOldestFrameTime;
	int32 MinNewestFrameTime;
	int32 MaxNewestFrameTime;

	const FSlateBrush* DarkBrush;
	const FSlateBrush* BrightBrush;

	const TSharedRef<FSlateFontMeasure> FontMeasureService;
	const FSlateFontInfo FontInfo;

	const FLinearColor ColorSynchronizedSamples;
	const FLinearColor ColorFutureSamples;
	const FLinearColor ColorPastSamples;

	TStrongObjectPtr<UTimecodeSynchronizer> TimecodeSynchronizer;
};

class STimecodeSynchronizerWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STimecodeSynchronizerWidget)
	{}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, UTimecodeSynchronizer& InTimecodeSynchronizer);

	void SetUpdateRate(int32 InUpdateRate);
	void SetFrameWidth(int32 InFrameWidth);

private:
	TSharedPtr<STimecodeSynchronizerBarWidget> BarWidget;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
