// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define TIMECODESYNCHRONIZEREDITOR_MODULE_NAME TEXT("TimecodeSynchronizerEditor")

class UE_DEPRECATED(5.0, "The TimecodeSynchronizer plugin is deprecated. Please update your project to use the features of the TimedDataMonitor plugin.") ITimecodeSynchronizerEditorModule;
class TIMECODESYNCHRONIZEREDITOR_API ITimecodeSynchronizerEditorModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to TimecodeSynchronizerEditor
	 * @return Returns TimecodeSynchronizerEditor singleton instance, loading the module on demand if needed
	 */
	static inline ITimecodeSynchronizerEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked<ITimecodeSynchronizerEditorModule>(TIMECODESYNCHRONIZEREDITOR_MODULE_NAME);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(TIMECODESYNCHRONIZEREDITOR_MODULE_NAME);
	}
};

#undef TIMECODESYNCHRONIZEREDITOR_MODULE_NAME
