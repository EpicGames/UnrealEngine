// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "TimecodeSynchronizerFactory.generated.h"

class UE_DEPRECATED(5.0, "The TimecodeSynchronizer plugin is deprecated. Please update your project to use the features of the TimedDataMonitor plugin.") UTimecodeSynchronizerFactory;
UCLASS(hidecategories = Object)
class TIMECODESYNCHRONIZEREDITOR_API UTimecodeSynchronizerFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface
};
