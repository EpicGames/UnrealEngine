// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TimecodeSynchronizerEditor : ModuleRules
{
	public TimecodeSynchronizerEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetTools",
				"Core",
				"CoreUObject",
				"EditorFramework",
				"Engine",		
				"InputCore",
                "MediaAssets",
                "MediaPlayerEditor",
                "PropertyEditor",
                "SlateCore",
				"Slate",
                "TimecodeSynchronizer",
                "UnrealEd",
			});
	}
}
