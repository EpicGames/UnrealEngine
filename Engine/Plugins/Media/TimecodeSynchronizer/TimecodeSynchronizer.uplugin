{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Timecode Synchronizer (Deprecated)",
	"Description": "This plugin has been deprecated and will be removed in a future engine version. Please update your project to use the features of the TimedDataMonitor plugin instead.\nAn asset that will become the TimecodeProvider once all the inputs get synchronized to a timecode.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "TimecodeSynchronizer",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "TimecodeSynchronizerEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "MediaPlayerEditor",
			"Enabled": true
		}
	],
	"IsExperimentalVersion": false
}